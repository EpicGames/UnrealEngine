{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Scriptable Tools Framework",
	"Description": "Blueprint-Scriptable extension to the Interactive Tools Framework",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ScriptableToolsFramework",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "EditorScriptableToolsFramework",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "MeshModelingToolset",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true
		}
	]
}
