// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class EditorScriptableToolsFramework : ModuleRules
{
	public EditorScriptableToolsFramework(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);


		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);


		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"PhysicsCore",
				"RenderCore",
				"GeometryCore",
				"InteractiveToolsFramework",
				"ScriptableToolsFramework"
			}
			);


		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"UnrealEd"
			}
			);


		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);

	}
}
