// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptableToolSet.h"

#include "ScriptableInteractiveTool.h"
#include "ScriptableToolBuilder.h"
#include "Tags/ScriptableToolGroupSet.h"
#include "Engine/AssetManager.h" // Singleton access to StreamableManager

#include "Utility/ScriptableToolLogging.h"

#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"

#if WITH_EDITOR
	#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolSet)

UScriptableToolSet::UScriptableToolSet()
{
	AssetCanDeleteHandle = FEditorDelegates::OnAssetsCanDelete.AddUObject(this, &UScriptableToolSet::HandleAssetCanDelete);
}

UScriptableToolSet::~UScriptableToolSet()
{
	FEditorDelegates::OnAssetsCanDelete.Remove(AssetCanDeleteHandle);

	UnloadAllTools();
}

void  UScriptableToolSet::UnloadAllTools()
{
	if (bActiveLoading)
	{
		AsyncLoadHandle->CancelHandle();
	}

	Tools.Reset();
	ToolBuilders.Reset();

	bActiveLoading = false;
}

void UScriptableToolSet::HandleAssetCanDelete(const TArray<UObject*>& InObjectsToDelete, FCanDeleteAssetResult& OutCanDelete)
{
	OutCanDelete.Set(true);

	for (UObject* Obj : InObjectsToDelete)
	{
		if (Tools.ContainsByPredicate([&Obj](const FScriptableToolInfo& ToolInfo) {
			return ToolInfo.ToolPath == Obj->GetPathName() || ToolInfo.BuilderPath == Obj->GetPathName();
			}))
		{
			UE_LOGF(LogScriptableTools, Error, "Unable to delete asset. Asset is currently loaded as part of current scriptable tool palette: %ls", *Obj->GetPathName());

			OutCanDelete.Set(false);
		}
	}
}

void UScriptableToolSet::ReinitializeScriptableTools(FPreToolsLoadedDelegate PreDelegate, FToolsLoadedDelegate PostDelegate, FToolsLoadingUpdateDelegate UpdateDelegate, FScriptableToolGroupSet* TagsToFilter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UScriptableToolSet::ReinitializeScriptableTools)

	if (bActiveLoading)
	{
		AsyncLoadHandle->CancelHandle();
	}

	PreDelegate.ExecuteIfBound();

	bActiveLoading = true;
	Tools.Reset();
	ToolBuilders.Reset();

	UClass* ScriptableToolClass = UScriptableInteractiveTool::StaticClass();
	UScriptableInteractiveTool* ScriptableToolCDO = ScriptableToolClass->GetDefaultObject<UScriptableInteractiveTool>();

	// Iterate over Blueprint classes to try to find UScriptableInteractiveTool blueprints
	// Note that this code may not be fully reliable, but it appears to work so far...

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	// The asset registry is populated asynchronously at startup, so there's no guarantee it has finished.
	// This simple approach just runs a synchronous scan on the entire content directory.
	// Better solutions would be to specify only the path to where the relevant blueprints are,
	// or to register a callback with the asset registry to be notified of when it's finished populating.
	TArray< FString > ContentPaths;
	ContentPaths.Add(TEXT("/Game"));
	AssetRegistry.ScanPathsSynchronous(ContentPaths);

	FTopLevelAssetPath BaseClassPath = ScriptableToolClass->GetClassPathName();

	// Use the asset registry to get the set of all class names deriving from Base
	TArray<FTopLevelAssetPath> BaseModeClasses;
	BaseModeClasses.Add(BaseClassPath);
	TSet<FTopLevelAssetPath> DerivedClassPaths;
	AssetRegistry.GetDerivedClassNames(BaseModeClasses, TSet<FTopLevelAssetPath>(), DerivedClassPaths);

	TArray< FSoftObjectPath > ObjectPathsToLoad;
	for (const FTopLevelAssetPath& ClassPath : DerivedClassPaths)
	{
		// don't include Base tools  (note this will also catch EditorScriptableToolsFramework)
		if (ClassPath.ToString().Contains(TEXT("ScriptableToolsFramework")))
		{
			continue;
		}

		ObjectPathsToLoad.Add(ClassPath.ToString());
	}
	
	TSharedPtr<FScriptableToolGroupSet> TagsToFilterCopy;
	if (TagsToFilter)
	{
		TagsToFilterCopy = MakeShared<FScriptableToolGroupSet>(*TagsToFilter);
	}

	AsyncLoadHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(ObjectPathsToLoad, [this, PostDelegate, ObjectPathsToLoad, TagsToFilterCopy]() { PostToolLoad(PostDelegate, ObjectPathsToLoad, TagsToFilterCopy);  });

	if (AsyncLoadHandle)
	{
		AsyncLoadHandle->BindUpdateDelegate(UpdateDelegate);
	}
	else
	{
		PostToolLoad(PostDelegate, ObjectPathsToLoad, TagsToFilterCopy);
	}
}




void UScriptableToolSet::ForEachScriptableTool(
	TFunctionRef<void(UClass* ToolClass, UBaseScriptableToolBuilder* ToolBuilder)> ProcessToolFunc)
{
	if (bActiveLoading)
	{
		return;
	}

	for (FScriptableToolInfo& ToolInfo : Tools)
	{
		if (ToolInfo.ToolClass.IsValid() && ToolInfo.ToolBuilder.IsValid())
		{
			ProcessToolFunc(ToolInfo.ToolClass.Get(), ToolInfo.ToolBuilder.Get());
		}
	}
}


void UScriptableToolSet::PostToolLoad(FToolsLoadedDelegate Delegate, TArray< FSoftObjectPath > ObjectsLoaded, TSharedPtr<FScriptableToolGroupSet> TagsToFilter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UScriptableToolSet::PostToolLoad)

	TSet<UClass*> PotentialToolClasses;
	UClass* ScriptableToolClass = UScriptableInteractiveTool::StaticClass();

	for(FSoftObjectPath& ObjectPath : ObjectsLoaded)
	{
		TSoftClassPtr<UScriptableInteractiveTool> SoftClass = TSoftClassPtr<UScriptableInteractiveTool>(ObjectPath);
		if (UClass* Class = SoftClass.LoadSynchronous())
		{
			if (Class->HasAnyClassFlags(CLASS_Abstract) || (Class->GetAuthoritativeClass() != Class))
			{
				continue;
			}

			if (TagsToFilter)
			{
				if (!TagsToFilter->Matches(Cast<UScriptableInteractiveTool>(Class->GetDefaultObject())->GroupTags))
				{
					continue;
				}
			}

			PotentialToolClasses.Add(Class);
		}
	}

	// if the class is viable, create a ToolBuilder for it
	for (UClass* Class : PotentialToolClasses)
	{
		if (Class->IsChildOf(ScriptableToolClass))
		{
			FScriptableToolInfo ToolInfo;
			ToolInfo.ToolPath = Class->GetPathName().LeftChop(2); // Remove the _C postfix - this lets us match against the asset later and we don't actually need it otherwise.
			ToolInfo.ToolClass = Class;
			ToolInfo.ToolCDO = Class->GetDefaultObject<UScriptableInteractiveTool>();

			if(!ToolInfo.ToolCDO->bShowToolInEditor)
			{
				continue; // If the tool doesn't want to be shown in editor, we can skip loading it entirely,
				          // preventing the deletion prevention from kicking in and spending cycles on tool builder tests
			}

			UBaseScriptableToolBuilder* ToolBuilder = ToolInfo.ToolCDO->GetNewCustomToolBuilderInstance(this);
			if (ToolBuilder == nullptr)
			{
				ToolBuilder = NewObject<UBaseScriptableToolBuilder>(this);
			}

			ToolBuilder->ToolClass = Class;
			ToolBuilders.Add(ToolBuilder);

			switch (ToolInfo.ToolCDO->ToolStartupRequirements)
			{
				// We need to get the path to the specific tool builder the tool is using. Despite appearances, this is how you do it.
				// Trying to do this directly on the ToolBuilder variable will result in the wrong path.
				case EScriptableToolStartupRequirements::ToolTarget:
				{					
					ToolInfo.BuilderPath = ToolInfo.ToolCDO->ToolTargetToolBuilderClass->GetPathName().LeftChop(2);
					break;
				}
				case EScriptableToolStartupRequirements::Custom:
				{
					ToolInfo.BuilderPath = ToolInfo.ToolCDO->CustomToolBuilderClass->GetPathName().LeftChop(2);
					break;
				}
			}
			
			ToolInfo.ToolBuilder = ToolBuilder;			

			Tools.Add(ToolInfo);
		}
	}

	bActiveLoading = false;
	Delegate.ExecuteIfBound();

	AsyncLoadHandle.Reset();
}


