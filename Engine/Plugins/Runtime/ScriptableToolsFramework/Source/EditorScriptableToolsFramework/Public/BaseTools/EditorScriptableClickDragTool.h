// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/ScriptableClickDragTool.h"
#include "EditorScriptableClickDragTool.generated.h"

/**
 * Editor-Only variant of UScriptableClickDragTool, which gives access to Editor-Only BP functions
 */
UCLASS(MinimalAPI, Transient, Blueprintable)
class UEditorScriptableClickDragTool : public UScriptableClickDragTool
{
	GENERATED_BODY()
public:

};
