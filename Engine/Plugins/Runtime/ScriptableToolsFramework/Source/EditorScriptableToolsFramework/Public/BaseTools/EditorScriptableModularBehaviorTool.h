// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/ScriptableModularBehaviorTool.h"
#include "EditorScriptableModularBehaviorTool.generated.h"

/**
 * Editor-Only variant of UScriptableModularBehaviorTool, which gives access to Editor-Only BP functions
 */
UCLASS(MinimalAPI, Transient, Blueprintable)
class UEditorScriptableModularBehaviorTool : public UScriptableModularBehaviorTool
{
	GENERATED_BODY()
public:

};
