// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/ScriptableSingleClickTool.h"
#include "EditorScriptableSingleClickTool.generated.h"

/**
 * Editor-Only variant of UScriptableSingleClickTool, which gives access to Editor-Only BP functions
 */
UCLASS(MinimalAPI, Transient, Blueprintable)
class UEditorScriptableSingleClickTool : public UScriptableSingleClickTool
{
	GENERATED_BODY()
public:

};
