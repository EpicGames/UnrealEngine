// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScriptableInteractiveTool.h"
#include "EditorScriptableInteractiveTool.generated.h"

/**
 * Editor-Only variant of UScriptableInteractiveTool, which gives access to Editor-Only BP functions
 */
UCLASS(MinimalAPI, Transient, Blueprintable)
class UEditorScriptableInteractiveTool : public UScriptableInteractiveTool
{
	GENERATED_BODY()
public:

};


UCLASS(MinimalAPI, Transient, Blueprintable)
class UEditorScriptableInteractiveToolPropertySet : public UScriptableInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

};
