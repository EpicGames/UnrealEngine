// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ScriptableToolSet.generated.h"

#define UE_API EDITORSCRIPTABLETOOLSFRAMEWORK_API

class UScriptableInteractiveTool;
class UBaseScriptableToolBuilder;
struct FScriptableToolGroupSet;
class UClass;
struct FCanDeleteAssetResult;

DECLARE_DELEGATE(FPreToolsLoadedDelegate);
DECLARE_DELEGATE(FToolsLoadedDelegate);
DECLARE_DELEGATE_OneParam(FToolsLoadingUpdateDelegate, TSharedRef<struct FStreamableHandle>);
/**
 * UScriptableToolSet represents a set of UScriptableInteractiveTool types.
 */
UCLASS(MinimalAPI)
class UScriptableToolSet : public UObject
{
	GENERATED_BODY()

public:

	UE_API UScriptableToolSet();

	UE_API virtual ~UScriptableToolSet();

	/**
	* Forces the unloading of all tools loaded
	*/
	UE_API void UnloadAllTools();

	/**
	 * Find all UScriptableInteractiveTool classes in the current project.
	 * (Currently no support for filtering/etc)
	 */
	UE_API void ReinitializeScriptableTools(FPreToolsLoadedDelegate PreDelegate, FToolsLoadedDelegate PostDelegate, FToolsLoadingUpdateDelegate UpdateDelegate, FScriptableToolGroupSet* TagsToFilter = nullptr);

	/**
	 * Allow external code to process each UScriptableInteractiveTool in the current ToolSet
	 */
	UE_API void ForEachScriptableTool(
		TFunctionRef<void(UClass* ToolClass, UBaseScriptableToolBuilder* ToolBuilder)> ProcessToolFunc);

private:

	UE_API void HandleAssetCanDelete(const TArray<UObject*>& InObjectsToDelete, FCanDeleteAssetResult& OutCanDelete);

	UE_API void PostToolLoad(FToolsLoadedDelegate Delegate, TArray< FSoftObjectPath > ObjectsLoaded, TSharedPtr<FScriptableToolGroupSet> TagsToFilter);

	bool bActiveLoading = false;
	TSharedPtr<FStreamableHandle> AsyncLoadHandle;

	FDelegateHandle AssetCanDeleteHandle;

	struct FScriptableToolInfo
	{
		FString ToolPath;
		FString BuilderPath;
		TWeakObjectPtr<UClass> ToolClass = nullptr;
		TWeakObjectPtr<UScriptableInteractiveTool> ToolCDO;
		TWeakObjectPtr<UBaseScriptableToolBuilder> ToolBuilder;
	};
	TArray<FScriptableToolInfo> Tools;

	UPROPERTY()
	TArray<TObjectPtr<UBaseScriptableToolBuilder>> ToolBuilders;
};

#undef UE_API
