// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/ScriptableClickDragTool.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableClickDragTool)

#define LOCTEXT_NAMESPACE "UScriptableClickDragTool"

void UScriptableClickDragTool::Setup()
{
	UScriptableInteractiveTool::Setup();

	// add default mouse input behavior
	ClickDragBehavior = NewObject<UClickDragInputBehavior>();
	ClickDragBehavior->Initialize(this);
	ClickDragBehavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	ClickDragBehavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	ClickDragBehavior->Modifiers.RegisterModifier(3, FInputDeviceState::IsAltKeyDown);
	ClickDragBehavior->bUpdateModifiersDuringDrag = this->bUpdateModifiersDuringDrag;
	AddInputBehavior(ClickDragBehavior);

	bShiftModifier = false;
	bCtrlModifier = false;
	bAltModifier = false;

	if (bWantMouseHover)
	{
		MouseHoverBehavior = NewObject<UMouseHoverBehavior>();
		MouseHoverBehavior->Initialize(this);
		MouseHoverBehavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
		MouseHoverBehavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
		MouseHoverBehavior->Modifiers.RegisterModifier(3, FInputDeviceState::IsAltKeyDown);
		AddInputBehavior(MouseHoverBehavior);
	}
}


FInputRayHit UScriptableClickDragTool::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	return TestIfCanBeginClickDrag(PressPos, GetActiveModifiers());
}

void UScriptableClickDragTool::OnClickPress(const FInputDeviceRay& PressPos)
{
	bInClickDrag = true;
	OnDragBegin(PressPos, GetActiveModifiers());
}

void UScriptableClickDragTool::OnClickDrag(const FInputDeviceRay& DragPos)
{
	OnDragUpdatePosition(DragPos, GetActiveModifiers());
}

void UScriptableClickDragTool::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	OnDragEnd(ReleasePos, GetActiveModifiers());
	bInClickDrag = false;
}

void UScriptableClickDragTool::OnTerminateDragSequence()
{
	OnDragSequenceCancelled();
	bInClickDrag = false;
}


bool UScriptableClickDragTool::InActiveClickDrag() const
{
	return bInClickDrag;
}


FInputRayHit UScriptableClickDragTool::TestIfCanBeginClickDrag_Implementation(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers)
{
	// always return a hit
	return FInputRayHit(0.0f);
}

void UScriptableClickDragTool::OnDragBegin_Implementation(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers)
{
}

void UScriptableClickDragTool::OnDragUpdatePosition_Implementation(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers)
{
}

void UScriptableClickDragTool::OnDragEnd_Implementation(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers)
{
}

void UScriptableClickDragTool::OnDragSequenceCancelled_Implementation()
{
}




void UScriptableClickDragTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == 1)
	{
		bShiftModifier = bIsOn;
	}
	if (ModifierID == 2)
	{
		bCtrlModifier = bIsOn;
	}
	if (ModifierID == 3)
	{
		bAltModifier = bIsOn;
	}
}

bool UScriptableClickDragTool::IsShiftDown() const
{
	return bShiftModifier;
}

bool UScriptableClickDragTool::IsCtrlDown() const
{
	return bCtrlModifier;
}

bool UScriptableClickDragTool::IsAltDown() const
{
	return bAltModifier;
}

FScriptableToolModifierStates UScriptableClickDragTool::GetActiveModifiers()
{
	FScriptableToolModifierStates Modifiers;
	Modifiers.bShiftDown = bShiftModifier;
	Modifiers.bCtrlDown = bCtrlModifier;
	Modifiers.bAltDown = bAltModifier;
	return Modifiers;
}




FInputRayHit UScriptableClickDragTool::OnHoverHitTest_Implementation(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers)
{
	// always return a hit
	return FInputRayHit(0.0f);
}
FInputRayHit UScriptableClickDragTool::BeginHoverSequenceHitTest(const FInputDeviceRay& ClickPos)
{
	return OnHoverHitTest(ClickPos, GetActiveModifiers());
}

void UScriptableClickDragTool::OnHoverBegin_Implementation(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers)
{
}

void UScriptableClickDragTool::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	bInHover = true;
	OnHoverBegin(DevicePos, GetActiveModifiers());
}

bool UScriptableClickDragTool::OnHoverUpdate_Implementation(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers)
{
	return true;
}

bool UScriptableClickDragTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	return OnHoverUpdate(DevicePos, GetActiveModifiers());
}


void UScriptableClickDragTool::OnHoverEnd_Implementation(const FScriptableToolModifierStates& Modifiers)
{
}

void UScriptableClickDragTool::OnEndHover()
{
	OnHoverEnd(GetActiveModifiers());
	bInHover = false;
}

bool UScriptableClickDragTool::InActiveHover() const
{
	return bInHover;
}



#undef LOCTEXT_NAMESPACE
