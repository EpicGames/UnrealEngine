// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/ScriptableModularBehaviorTool.h"

#include "Behaviors/ScriptableToolSingleClickBehavior.h"
#include "Behaviors/ScriptableToolClickDragBehavior.h"
#include "Behaviors/ScriptableToolDoubleClickBehavior.h"
#include "Behaviors/ScriptableToolMouseHoverBehavior.h"
#include "Behaviors/ScriptableToolMouseWheelBehavior.h"
#include "Behaviors/ScriptableToolMultiClickSequenceBehavior.h"
#include "Behaviors/ScriptableToolSingleClickOrDragBehavior.h"
#include "Behaviors/ScriptableToolKeyInputBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableModularBehaviorTool)

void UScriptableModularBehaviorTool::AddSingleClickBehavior(
	const FTestIfHitByClickDelegate TestIfHitByClickDelegate,
	const FOnHitByClickDelegate OnHitByClickDelegate,
	const FMouseBehaviorModiferCheckDelegate ModifierCheckFunction,
	int CapturePriority,
	EScriptableToolMouseButton MouseButton,
	bool bHitTestOnRelease
)
{
	TObjectPtr<UScriptableToolSingleClickBehavior> BehaviorContainer = NewObject<UScriptableToolSingleClickBehavior>();
	BehaviorContainer->Init(this, ModifierCheckFunction, TestIfHitByClickDelegate, OnHitByClickDelegate, MouseButton, bHitTestOnRelease);
	BehaviorContainer->SetDefaultPriority(FInputCapturePriority(CapturePriority));

	SingleClickBehaviors.Add(BehaviorContainer);
}


void UScriptableModularBehaviorTool::AddDoubleClickBehavior(
	const FTestIfHitByClickDelegate IfHitByClick,
	const FOnHitByClickDelegate OnHitByClick,
	const FMouseBehaviorModiferCheckDelegate ModifierCheckFunction,
	int CapturePriority,
	EScriptableToolMouseButton MouseButton,
	bool bHitTestOnRelease)
{
	TObjectPtr<UScriptableToolDoubleClickBehavior> BehaviorContainer = NewObject<UScriptableToolDoubleClickBehavior>();
	BehaviorContainer->Init(this, ModifierCheckFunction, IfHitByClick, OnHitByClick, MouseButton, bHitTestOnRelease);
	BehaviorContainer->SetDefaultPriority(FInputCapturePriority(CapturePriority));

	DoubleClickBehaviors.Add(BehaviorContainer);
}


void UScriptableModularBehaviorTool::AddClickDragBehavior(
	const FTestCanBeginClickDragSequenceDelegate TestCanBeginClickDragSequenceFuncIn,
	const FOnClickPressDelegate OnClickPressFuncIn,
	const FOnClickDragDelegate OnClickDragFuncIn,
	const FOnClickReleaseDelegate OnClickReleaseFuncIn,
	const FOnTerminateDragSequenceDelegate OnTerminateDragSequenceFuncIn,
	const FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
	int CapturePriority,
	EScriptableToolMouseButton MouseButtonIn,
	bool bUpdateModifiersDuringDrag
)
{
	TObjectPtr<UScriptableToolClickDragBehavior> BehaviorContainer = NewObject<UScriptableToolClickDragBehavior>();
	BehaviorContainer->Init(this, ModifierCheckFuncIn, TestCanBeginClickDragSequenceFuncIn, OnClickPressFuncIn, OnClickDragFuncIn, OnClickReleaseFuncIn, OnTerminateDragSequenceFuncIn, MouseButtonIn, bUpdateModifiersDuringDrag);
	BehaviorContainer->SetDefaultPriority(FInputCapturePriority(CapturePriority));

	ClickDragBehaviors.Add(BehaviorContainer);
}


void UScriptableModularBehaviorTool::AddSingleClickOrDragBehavior(
	FTestIfHitByClickDelegate TestIfHitByClickFuncIn,
	FOnHitByClickDelegate OnHitByClickFuncIn,
	FTestCanBeginClickDragSequenceDelegate TestCanBeginClickDragSequenceFuncIn,
	FOnClickPressDelegate OnClickPressFuncIn,
	FOnClickDragDelegate OnClickDragFuncIn,
	FOnClickReleaseDelegate OnClickReleaseFuncIn,
	FOnTerminateDragSequenceDelegate OnTerminateDragSequenceFuncIn,
	FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
	int CapturePriority,
	EScriptableToolMouseButton MouseButtonIn,
	bool bBeginDragIfClickTargetNotHit,
	float ClickDistanceThreshold
)
{
	TObjectPtr<UScriptableToolSingleClickOrDragBehavior> BehaviorContainer = NewObject<UScriptableToolSingleClickOrDragBehavior>();
	BehaviorContainer->Init(this, ModifierCheckFuncIn, TestIfHitByClickFuncIn, OnHitByClickFuncIn,
		                    TestCanBeginClickDragSequenceFuncIn, OnClickPressFuncIn, OnClickDragFuncIn,
		                    OnClickReleaseFuncIn, OnTerminateDragSequenceFuncIn, MouseButtonIn, 
		                    bBeginDragIfClickTargetNotHit, ClickDistanceThreshold);
	BehaviorContainer->SetDefaultPriority(FInputCapturePriority(CapturePriority));

	SingleClickOrDragBehaviors.Add(BehaviorContainer);
}



void UScriptableModularBehaviorTool::AddMouseWheelBehavior(
	FTestShouldRespondToMouseWheelDelegate TestShouldRespondToMouseWheelFuncIn,
	FOnMouseWheelScrollUpDelegate OnMouseWheelScrollUpFuncIn,
	FOnMouseWheelScrollDownDelegate OnMouseWheelScrollDownFuncIn,
	FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
	int CapturePriority
)
{
	TObjectPtr<UScriptableToolMouseWheelBehavior> BehaviorContainer = NewObject<UScriptableToolMouseWheelBehavior>();
	BehaviorContainer->Init(this, ModifierCheckFuncIn, TestShouldRespondToMouseWheelFuncIn, OnMouseWheelScrollUpFuncIn, OnMouseWheelScrollDownFuncIn);
	BehaviorContainer->SetDefaultPriority(FInputCapturePriority(CapturePriority));

	MouseWheelBehaviors.Add(BehaviorContainer);
}



void UScriptableModularBehaviorTool::AddMultiClickSequenceBehavior(
	FOnBeginSequencePreviewDelegate OnBeginSequencePreviewFuncIn,
	FCanBeginClickSequenceDelegate CanBeginClickSequenceFuncIn,
	FOnBeginClickSequenceDelegate OnBeginClickSequenceFuncIn,
	FOnNextSequencePreviewDelegate OnNextSequencePreviewFuncIn,
	FOnNextSequenceClickDelegate OnNextSequenceClickFuncIn,
	FOnTerminateClickSequenceDelegate OnTerminateClickSequenceFuncIn,
	FRequestAbortClickSequenceDelegate RequestAbortClickSequenceFuncIn,
	FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
	FMouseBehaviorModiferCheckDelegate HoverModifierCheckFuncIn,
	int CapturePriority,
	EScriptableToolMouseButton MouseButtonIn
)
{
	TObjectPtr<UScriptableToolClickSequenceBehavior> BehaviorContainer = NewObject<UScriptableToolClickSequenceBehavior>();
	BehaviorContainer->Init(this, ModifierCheckFuncIn, HoverModifierCheckFuncIn, OnBeginSequencePreviewFuncIn, CanBeginClickSequenceFuncIn, OnBeginClickSequenceFuncIn, OnNextSequencePreviewFuncIn,
		                    OnNextSequenceClickFuncIn, OnTerminateClickSequenceFuncIn, RequestAbortClickSequenceFuncIn, MouseButtonIn);
	BehaviorContainer->SetDefaultPriority(FInputCapturePriority(CapturePriority));

	MultiClickSequenceBehaviors.Add(BehaviorContainer);
}




void UScriptableModularBehaviorTool::AddMouseHoverBehavior(
	FBeginHoverSequenceHitTestDelegate BeginHoverSequenceHitTestFuncIn,
	FOnBeginHoverDelegate OnBeginHoverFuncIn,
	FOnUpdateHoverDelegate OnUpdateHoverFuncIn,
	FOnEndHoverDelegate OnEndHoverFuncIn,
	FMouseBehaviorModiferCheckDelegate HoverModifierCheckFuncIn,
	int CapturePriority
)
{
	TObjectPtr<UScriptableToolMouseHoverBehavior> BehaviorContainer = NewObject<UScriptableToolMouseHoverBehavior>();
	BehaviorContainer->Init(this, HoverModifierCheckFuncIn, BeginHoverSequenceHitTestFuncIn, OnBeginHoverFuncIn, OnUpdateHoverFuncIn, OnEndHoverFuncIn);
	BehaviorContainer->SetDefaultPriority(FInputCapturePriority(CapturePriority));

	MouseHoverBehaviors.Add(BehaviorContainer);
}


void UScriptableModularBehaviorTool::AddSingleKeyInputBehavior(
	FOnKeyStateToggleDelegate OnKeyPressedFuncIn,
	FOnKeyStateToggleDelegate OnKeyReleasedFuncIn,
	FOnForceEndCaptureDelegate_ScriptableTools OnForceEndCaptureFuncIn,
	FKey Key,
	const FMouseBehaviorModiferCheckDelegate ModifierCheckFunction,
	int CapturePriority
)
{
	TArray<FKey> Keys;
	Keys.Add(Key);

	TObjectPtr<UScriptableToolKeyInputBehavior> BehaviorContainer = NewObject<UScriptableToolKeyInputBehavior>();
	BehaviorContainer->Init(this, ModifierCheckFunction, OnKeyPressedFuncIn, OnKeyReleasedFuncIn, OnForceEndCaptureFuncIn, Keys, true);
	BehaviorContainer->SetDefaultPriority(FInputCapturePriority(CapturePriority));

	KeyInputBehaviors.Add(BehaviorContainer);
}


void UScriptableModularBehaviorTool::AddMultiKeyInputBehavior(
	FOnKeyStateToggleDelegate OnKeyPressedFuncIn,
	FOnKeyStateToggleDelegate OnKeyReleasedFuncIn,
	FOnForceEndCaptureDelegate_ScriptableTools OnForceEndCaptureFuncIn,
	TArray<FKey> Keys,
	bool bRequireAllKeys,
	const FMouseBehaviorModiferCheckDelegate ModifierCheckFunction,
	int CapturePriority
)
{
	TObjectPtr<UScriptableToolKeyInputBehavior> BehaviorContainer = NewObject<UScriptableToolKeyInputBehavior>();
	BehaviorContainer->Init(this, ModifierCheckFunction, OnKeyPressedFuncIn, OnKeyReleasedFuncIn, OnForceEndCaptureFuncIn, Keys, bRequireAllKeys);
	BehaviorContainer->SetDefaultPriority(FInputCapturePriority(CapturePriority));

	KeyInputBehaviors.Add(BehaviorContainer);
}

void UScriptableModularBehaviorTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == 1)
	{
		bShiftModifier = bIsOn;
	}
	if (ModifierID == 2)
	{
		bCtrlModifier = bIsOn;
	}
	if (ModifierID == 3)
	{
		bAltModifier = bIsOn;
	}
}

bool UScriptableModularBehaviorTool::IsShiftDown() const
{
	return bShiftModifier;
}

bool UScriptableModularBehaviorTool::IsCtrlDown() const
{
	return bCtrlModifier;
}

bool UScriptableModularBehaviorTool::IsAltDown() const
{
	return bAltModifier;
}

FScriptableToolModifierStates UScriptableModularBehaviorTool::GetActiveModifiers()
{
	FScriptableToolModifierStates Modifiers;
	Modifiers.bShiftDown = bShiftModifier;
	Modifiers.bCtrlDown = bCtrlModifier;
	Modifiers.bAltDown = bAltModifier;
	return Modifiers;
}


