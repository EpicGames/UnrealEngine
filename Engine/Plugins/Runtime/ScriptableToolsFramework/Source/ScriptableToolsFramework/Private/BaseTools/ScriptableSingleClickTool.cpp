// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/ScriptableSingleClickTool.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableSingleClickTool)


#define LOCTEXT_NAMESPACE "UScriptableSingleClickTool"

void UScriptableSingleClickTool::Setup()
{
	UScriptableInteractiveTool::Setup();

	// add default button input behaviors for devices
	SingleClickBehavior = NewObject<USingleClickInputBehavior>();
	SingleClickBehavior->Initialize(this);
	AddInputBehavior(SingleClickBehavior);

	SingleClickBehavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	SingleClickBehavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	SingleClickBehavior->Modifiers.RegisterModifier(3, FInputDeviceState::IsAltKeyDown);
	bShiftModifier = false;
	bCtrlModifier = false;
	bAltModifier = false;


	if (bWantMouseHover)
	{
		MouseHoverBehavior = NewObject<UMouseHoverBehavior>();
		MouseHoverBehavior->Initialize(this);
		AddInputBehavior(MouseHoverBehavior);
	}

}


void UScriptableSingleClickTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == 1)
	{
		bShiftModifier = bIsOn;
	}
	if (ModifierID == 2)
	{
		bCtrlModifier = bIsOn;
	}
	if (ModifierID == 3)
	{
		bAltModifier = bIsOn;
	}
}

bool UScriptableSingleClickTool::IsShiftDown() const
{
	return bShiftModifier;
}

bool UScriptableSingleClickTool::IsCtrlDown() const
{
	return bCtrlModifier;
}

bool UScriptableSingleClickTool::IsAltDown() const
{
	return bAltModifier;
}

FScriptableToolModifierStates UScriptableSingleClickTool::GetActiveModifiers()
{
	FScriptableToolModifierStates Modifiers;
	Modifiers.bShiftDown = bShiftModifier;
	Modifiers.bCtrlDown = bCtrlModifier;
	Modifiers.bAltDown = bAltModifier;
	return Modifiers;
}


FInputRayHit UScriptableSingleClickTool::TestIfHitByClick_Implementation(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers)
{
	UE_LOGF(LogTemp, Warning, "UScriptableSingleClickTool: Default TestIfHitByClick Implementation always consumes hit");

	// always return a hit
	return FInputRayHit(0.0f);
}
FInputRayHit UScriptableSingleClickTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	return TestIfHitByClick(ClickPos, GetActiveModifiers());
}

void UScriptableSingleClickTool::OnHitByClick_Implementation(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers)
{
	UE_LOGF(LogTemp, Warning, "UScriptableSingleClickTool: Empty OnHitByClick Implementation");
}
void UScriptableSingleClickTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	OnHitByClick(ClickPos, GetActiveModifiers());
}


FInputRayHit UScriptableSingleClickTool::OnHoverHitTest_Implementation(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers)
{
	// always return a hit
	return FInputRayHit(0.0f);
}
FInputRayHit UScriptableSingleClickTool::BeginHoverSequenceHitTest(const FInputDeviceRay& ClickPos)
{
	return OnHoverHitTest(ClickPos, GetActiveModifiers());
}

void UScriptableSingleClickTool::OnHoverBegin_Implementation(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers)
{
}

void UScriptableSingleClickTool::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	bInHover = true;
	OnHoverBegin(DevicePos, GetActiveModifiers());
}

bool UScriptableSingleClickTool::OnHoverUpdate_Implementation(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers)
{
	return true;
}

bool UScriptableSingleClickTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	return OnHoverUpdate(DevicePos, GetActiveModifiers());
}


void UScriptableSingleClickTool::OnHoverEnd_Implementation(const FScriptableToolModifierStates& Modifiers)
{
}

void UScriptableSingleClickTool::OnEndHover()
{
	OnHoverEnd(GetActiveModifiers());
	bInHover = false;
}

bool UScriptableSingleClickTool::InActiveHover() const
{
	return bInHover;
}


#undef LOCTEXT_NAMESPACE
