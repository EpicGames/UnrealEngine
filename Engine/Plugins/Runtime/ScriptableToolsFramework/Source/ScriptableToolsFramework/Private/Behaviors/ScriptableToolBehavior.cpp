// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptableToolBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolBehavior)

void UScriptableToolBehavior::SetDefaultPriority(const FInputCapturePriority& Priority)
{
	GetWrappedBehavior()->SetDefaultPriority(Priority);
}


