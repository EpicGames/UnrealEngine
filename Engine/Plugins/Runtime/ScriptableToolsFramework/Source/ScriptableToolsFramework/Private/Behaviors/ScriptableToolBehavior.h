// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InputBehavior.h"

#include "ScriptableToolBehavior.generated.h"

UCLASS(Abstract)
class UScriptableToolBehavior : public UObject
{
	GENERATED_BODY()

public:

	UScriptableToolBehavior() {};

	void SetDefaultPriority(const FInputCapturePriority& Priority);

	virtual UInputBehavior* GetWrappedBehavior() PURE_VIRTUAL(UScriptableToolBehavior::GetWrappedBehavior, return nullptr;);

};