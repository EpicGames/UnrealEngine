// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/ScriptableToolClickDragBehavior.h"

#include "BaseTools/ScriptableModularBehaviorTool.h"
#include "BaseBehaviors/ClickDragBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolClickDragBehavior)


void UScriptableToolClickDragBehavior::Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
	FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
	FTestCanBeginClickDragSequenceDelegate TestCanBeginClickDragSequenceFuncIn,
	FOnClickPressDelegate OnClickPressFuncIn,
	FOnClickDragDelegate OnClickDragFuncIn,
	FOnClickReleaseDelegate OnClickReleaseFuncIn,
	FOnTerminateDragSequenceDelegate OnTerminateDragSequenceFuncIn,
	EScriptableToolMouseButton MouseButtonIn,
	bool bUpdateModifiersDuringDragIn)
{
	BehaviorHost = BehaviorHostIn;
	Behavior = NewObject<UClickDragInputBehavior>();
	ModifierCheckFunc = ModifierCheckFuncIn;
	TestCanBeginClickDragSequenceFunc = TestCanBeginClickDragSequenceFuncIn;
	OnClickPressFunc = OnClickPressFuncIn;
	OnClickDragFunc = OnClickDragFuncIn;
	OnClickReleaseFunc = OnClickReleaseFuncIn;
	OnTerminateDragSequenceFunc = OnTerminateDragSequenceFuncIn;
	MouseButton = MouseButtonIn;

	Behavior->Initialize(this);
	Behavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState) {
		bool bResult = true;
		if (ModifierCheckFunc.IsBound())
		{
			bResult = ModifierCheckFunc.Execute(InputDeviceState);
		}
		return bResult;
	};

	Behavior->bUpdateModifiersDuringDrag = bUpdateModifiersDuringDragIn;

	BehaviorHost->AddInputBehavior(Behavior);

	Behavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	Behavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	Behavior->Modifiers.RegisterModifier(3, FInputDeviceState::IsAltKeyDown);

	switch (MouseButton)
	{
	case EScriptableToolMouseButton::LeftButton:
	{
		Behavior->SetUseLeftMouseButton();
		break;
	}
	case EScriptableToolMouseButton::RightButton:
	{
		Behavior->SetUseRightMouseButton();
		break;
	}
	case EScriptableToolMouseButton::MiddleButton:
	{
		Behavior->SetUseMiddleMouseButton();
		break;
	}
	default:
		ensure(false);
	}
}

FInputRayHit UScriptableToolClickDragBehavior::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FInputRayHit RayHit;
	if (TestCanBeginClickDragSequenceFunc.IsBound())
	{
		RayHit = TestCanBeginClickDragSequenceFunc.Execute(PressPos, BehaviorHost->GetActiveModifiers(), MouseButton);
	}
	return RayHit;
}

UInputBehavior* UScriptableToolClickDragBehavior::GetWrappedBehavior()
{
	return Behavior;
}

void UScriptableToolClickDragBehavior::OnClickPress(const FInputDeviceRay& PressPos)
{
	OnClickPressFunc.ExecuteIfBound(PressPos, BehaviorHost->GetActiveModifiers(), MouseButton);
}

void UScriptableToolClickDragBehavior::OnClickDrag(const FInputDeviceRay& DragPos)
{
	OnClickDragFunc.ExecuteIfBound(DragPos, BehaviorHost->GetActiveModifiers(), MouseButton);
}

void UScriptableToolClickDragBehavior::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	OnClickReleaseFunc.ExecuteIfBound(ReleasePos, BehaviorHost->GetActiveModifiers(), MouseButton);
}

void UScriptableToolClickDragBehavior::OnTerminateDragSequence()
{
	OnTerminateDragSequenceFunc.ExecuteIfBound(BehaviorHost->GetActiveModifiers(), MouseButton);
}

void UScriptableToolClickDragBehavior::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	BehaviorHost->OnUpdateModifierState(ModifierID, bIsOn);
}
