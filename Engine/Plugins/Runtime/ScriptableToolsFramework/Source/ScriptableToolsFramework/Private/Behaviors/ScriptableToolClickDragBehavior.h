// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviors/ScriptableToolBehaviorDelegates.h"
#include "ScriptableToolBehavior.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "ScriptableToolClickDragBehavior.generated.h"

class UScriptableModularBehaviorTool;
class UClickDragInputBehavior;


UCLASS()
class UScriptableToolClickDragBehavior : public UScriptableToolBehavior, public IClickDragBehaviorTarget
{
	GENERATED_BODY()

public:
	UScriptableToolClickDragBehavior() {};

	void Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
		FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
		FTestCanBeginClickDragSequenceDelegate TestCanBeginClickDragSequenceFuncIn,
		FOnClickPressDelegate OnClickPressFuncIn,
		FOnClickDragDelegate OnClickDragFuncIn,
		FOnClickReleaseDelegate OnClickReleaseFuncIn,
		FOnTerminateDragSequenceDelegate OnTerminateDragSequenceFuncIn,
		EScriptableToolMouseButton MouseButtonIn,
		bool bUpdateModifiersDuringDragIn);

	virtual UInputBehavior* GetWrappedBehavior() override;

	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	virtual void OnTerminateDragSequence() override;
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

private:

	UPROPERTY()
	TObjectPtr<UScriptableModularBehaviorTool> BehaviorHost;

	UPROPERTY()
	TObjectPtr<UClickDragInputBehavior> Behavior;

	UPROPERTY()
	FMouseBehaviorModiferCheckDelegate ModifierCheckFunc;

	UPROPERTY()
	FTestCanBeginClickDragSequenceDelegate	TestCanBeginClickDragSequenceFunc;

	UPROPERTY()
	FOnClickPressDelegate OnClickPressFunc;

	UPROPERTY()
	FOnClickDragDelegate OnClickDragFunc;

	UPROPERTY()
	FOnClickReleaseDelegate OnClickReleaseFunc;

	UPROPERTY()
	FOnTerminateDragSequenceDelegate OnTerminateDragSequenceFunc;

	UPROPERTY()
	EScriptableToolMouseButton MouseButton;
};