// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScriptableToolSingleClickBehavior.h"

#include "ScriptableToolDoubleClickBehavior.generated.h"

UCLASS()
class UScriptableToolDoubleClickBehavior : public UScriptableToolSingleClickBehavior
{
	GENERATED_BODY()

public:
	UScriptableToolDoubleClickBehavior() {};

	virtual USingleClickInputBehavior* CreateNewBehavior() const override;
};
