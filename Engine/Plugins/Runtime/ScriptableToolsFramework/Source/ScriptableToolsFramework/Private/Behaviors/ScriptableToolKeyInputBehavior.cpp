// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/ScriptableToolKeyInputBehavior.h"

#include "BaseTools/ScriptableModularBehaviorTool.h"
#include "BaseBehaviors/KeyInputBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolKeyInputBehavior)

void UScriptableToolKeyInputBehavior::Init(
	TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
	FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
	FOnKeyStateToggleDelegate OnKeyPressedFuncIn,
	FOnKeyStateToggleDelegate OnKeyReleasedFuncIn,
	FOnForceEndCaptureDelegate_ScriptableTools OnForceEndCaptureFuncIn,
	const TArray<FKey>& ListenKeysIn,
	bool bRequireAllKeysIn
)
{
	BehaviorHost = BehaviorHostIn;
	Behavior = CreateNewBehavior();
	ModifierCheckFunc = ModifierCheckFuncIn;
	OnKeyPressedFunc = OnKeyPressedFuncIn;
	OnKeyReleasedFunc = OnKeyReleasedFuncIn;
	OnForceEndCaptureFunc = OnForceEndCaptureFuncIn;
	ListenKeys = ListenKeysIn;

	Behavior->Initialize(this, ListenKeys);
	Behavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState) {
		bool bResult = true;
		if (ModifierCheckFunc.IsBound())
		{
			bResult = ModifierCheckFunc.Execute(InputDeviceState);
		}
		return bResult;
	};
	Behavior->bRequireAllKeys = bRequireAllKeysIn;

	BehaviorHost->AddInputBehavior(Behavior);

	Behavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	Behavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	Behavior->Modifiers.RegisterModifier(3, FInputDeviceState::IsAltKeyDown);

}

UKeyInputBehavior* UScriptableToolKeyInputBehavior::CreateNewBehavior() const
{
	return NewObject<UKeyInputBehavior>();
}

UInputBehavior* UScriptableToolKeyInputBehavior::GetWrappedBehavior()
{
	return Behavior;
}

void UScriptableToolKeyInputBehavior::OnKeyPressed(const FKey& Key)
{
	if (OnKeyPressedFunc.IsBound())
	{
		OnKeyPressedFunc.Execute(Key, BehaviorHost->GetActiveModifiers());
	}
}

void UScriptableToolKeyInputBehavior::OnKeyReleased(const FKey& Key)
{
	if (OnKeyReleasedFunc.IsBound())
	{
		OnKeyReleasedFunc.Execute(Key, BehaviorHost->GetActiveModifiers());
	}
}

void UScriptableToolKeyInputBehavior::OnForceEndCapture()
{
	if (OnForceEndCaptureFunc.IsBound())
	{
		OnForceEndCaptureFunc.Execute();
	}
}

void UScriptableToolKeyInputBehavior::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	BehaviorHost->OnUpdateModifierState(ModifierID, bIsOn);
}
