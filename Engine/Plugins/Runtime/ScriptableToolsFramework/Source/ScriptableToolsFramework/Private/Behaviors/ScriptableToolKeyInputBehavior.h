// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviors/ScriptableToolBehaviorDelegates.h"
#include "ScriptableToolBehavior.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "ScriptableToolKeyInputBehavior.generated.h"

class UScriptableModularBehaviorTool;
class UKeyInputBehavior;

UCLASS()
class UScriptableToolKeyInputBehavior : public UScriptableToolBehavior, public IKeyInputBehaviorTarget
{
	GENERATED_BODY()

public:
	UScriptableToolKeyInputBehavior() {};

	void Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
		FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
		FOnKeyStateToggleDelegate OnKeyPressedFuncIn,
		FOnKeyStateToggleDelegate OnKeyReleasedFuncIn,
		FOnForceEndCaptureDelegate_ScriptableTools OnForceEndCaptureFuncIn,
		const TArray<FKey>& ListenKeysIn,
		bool bRequireAllKeys);

	virtual UKeyInputBehavior* CreateNewBehavior() const;
	virtual UInputBehavior* GetWrappedBehavior() override;

	virtual void OnKeyPressed(const FKey& KeyID) override;
	virtual void OnKeyReleased(const FKey& KeyID) override;
	virtual void OnForceEndCapture() override;

	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

private:

	UPROPERTY()
	TObjectPtr<UScriptableModularBehaviorTool> BehaviorHost;

	UPROPERTY()
	TObjectPtr<UKeyInputBehavior> Behavior;

	UPROPERTY()
	FMouseBehaviorModiferCheckDelegate ModifierCheckFunc;

	UPROPERTY()
	FOnKeyStateToggleDelegate	OnKeyPressedFunc;

	UPROPERTY()
	FOnKeyStateToggleDelegate OnKeyReleasedFunc;

	UPROPERTY()
	FOnForceEndCaptureDelegate_ScriptableTools OnForceEndCaptureFunc;

	UPROPERTY()
	TArray<FKey> ListenKeys;
};
