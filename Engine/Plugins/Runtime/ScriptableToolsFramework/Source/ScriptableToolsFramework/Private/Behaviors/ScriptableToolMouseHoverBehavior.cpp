// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/ScriptableToolMouseHoverBehavior.h"

#include "BaseTools/ScriptableModularBehaviorTool.h"
#include "BaseBehaviors/MouseHoverBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolMouseHoverBehavior)


void UScriptableToolMouseHoverBehavior::Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
	FMouseBehaviorModiferCheckDelegate HoverModifierCheckFuncIn,
	FBeginHoverSequenceHitTestDelegate BeginHoverSequenceHitTestFuncIn,
	FOnBeginHoverDelegate OnBeginHoverFuncIn,
	FOnUpdateHoverDelegate OnUpdateHoverFuncIn,
	FOnEndHoverDelegate OnEndHoverFuncIn)
{
	BehaviorHost = BehaviorHostIn;
	Behavior = NewObject<UMouseHoverBehavior>();
	HoverModifierCheckFunc = HoverModifierCheckFuncIn;
	BeginHoverSequenceHitTestFunc = BeginHoverSequenceHitTestFuncIn;
	OnBeginHoverFunc = OnBeginHoverFuncIn;
	OnUpdateHoverFunc = OnUpdateHoverFuncIn;
	OnEndHoverFunc = OnEndHoverFuncIn;

	Behavior->Initialize(this);

	Behavior->HoverModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState)
	{
		bool bResult = true;
		if (HoverModifierCheckFunc.IsBound())
		{
			bResult = HoverModifierCheckFunc.Execute(InputDeviceState);
		}
		return bResult;
	};

	BehaviorHost->AddInputBehavior(Behavior);

	Behavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	Behavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	Behavior->Modifiers.RegisterModifier(3, FInputDeviceState::IsAltKeyDown);
}

FInputRayHit UScriptableToolMouseHoverBehavior::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit RayHit;
	if (BeginHoverSequenceHitTestFunc.IsBound())
	{
		RayHit = BeginHoverSequenceHitTestFunc.Execute(PressPos, BehaviorHost->GetActiveModifiers());
	}
	return RayHit;
}

UInputBehavior* UScriptableToolMouseHoverBehavior::GetWrappedBehavior()
{
	return Behavior;
}

void UScriptableToolMouseHoverBehavior::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	OnBeginHoverFunc.ExecuteIfBound(DevicePos, BehaviorHost->GetActiveModifiers());
}

bool UScriptableToolMouseHoverBehavior::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	bool bShouldContinue = false;
	if (OnUpdateHoverFunc.IsBound())
	{
		bShouldContinue = OnUpdateHoverFunc.Execute(DevicePos, BehaviorHost->GetActiveModifiers());
	}
	return bShouldContinue;
}

void UScriptableToolMouseHoverBehavior::OnEndHover()
{
	OnEndHoverFunc.ExecuteIfBound();
}

void UScriptableToolMouseHoverBehavior::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	BehaviorHost->OnUpdateModifierState(ModifierID, bIsOn);
}
