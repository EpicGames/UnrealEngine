// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviors/ScriptableToolBehaviorDelegates.h"
#include "ScriptableToolBehavior.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "ScriptableToolMouseHoverBehavior.generated.h"

class UScriptableModularBehaviorTool;
class UMouseHoverBehavior;


UCLASS()
class UScriptableToolMouseHoverBehavior : public UScriptableToolBehavior, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	UScriptableToolMouseHoverBehavior() {};

	void Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
		FMouseBehaviorModiferCheckDelegate HoverModifierCheckFuncIn,
		FBeginHoverSequenceHitTestDelegate BeginHoverSequenceHitTestFuncIn,
		FOnBeginHoverDelegate OnBeginHoverFuncIn,
		FOnUpdateHoverDelegate OnUpdateHoverFuncIn,
		FOnEndHoverDelegate OnEndHoverFuncIn);

	virtual UInputBehavior* GetWrappedBehavior() override;

	virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override;
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

private:

	UPROPERTY()
	TObjectPtr<UScriptableModularBehaviorTool> BehaviorHost;

	UPROPERTY()
	TObjectPtr<UMouseHoverBehavior> Behavior;

	UPROPERTY()
	FMouseBehaviorModiferCheckDelegate HoverModifierCheckFunc;

	UPROPERTY()
	FBeginHoverSequenceHitTestDelegate BeginHoverSequenceHitTestFunc;

	UPROPERTY()
	FOnBeginHoverDelegate OnBeginHoverFunc;

	UPROPERTY()
	FOnUpdateHoverDelegate OnUpdateHoverFunc;

	UPROPERTY()
	FOnEndHoverDelegate OnEndHoverFunc;

};