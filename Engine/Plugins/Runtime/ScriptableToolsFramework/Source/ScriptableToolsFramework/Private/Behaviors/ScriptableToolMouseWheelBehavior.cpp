// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/ScriptableToolMouseWheelBehavior.h"

#include "BaseTools/ScriptableModularBehaviorTool.h"
#include "BaseBehaviors/MouseWheelBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolMouseWheelBehavior)


void UScriptableToolMouseWheelBehavior::Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
	FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
	FTestShouldRespondToMouseWheelDelegate TestShouldRespondToMouseWheelFuncIn,
	FOnMouseWheelScrollUpDelegate OnMouseWheelScrollUpFuncIn,
	FOnMouseWheelScrollDownDelegate OnMouseWheelScrollDownFuncIn)
{
	BehaviorHost = BehaviorHostIn;
	Behavior = NewObject<UMouseWheelInputBehavior>();
	ModifierCheckFunc = ModifierCheckFuncIn;
	TestShouldRespondToMouseWheelFunc = TestShouldRespondToMouseWheelFuncIn;
	OnMouseWheelScrollUpFunc = OnMouseWheelScrollUpFuncIn;
	OnMouseWheelScrollDownFunc = OnMouseWheelScrollDownFuncIn;

	Behavior->Initialize(this);
	Behavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState) {
		bool bResult = true;
		if (ModifierCheckFunc.IsBound())
		{
			bResult = ModifierCheckFunc.Execute(InputDeviceState);
		}
		return bResult;
	};


	BehaviorHost->AddInputBehavior(Behavior);

	Behavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	Behavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	Behavior->Modifiers.RegisterModifier(3, FInputDeviceState::IsAltKeyDown);

}

UInputBehavior* UScriptableToolMouseWheelBehavior::GetWrappedBehavior()
{
	return Behavior;
}

FInputRayHit UScriptableToolMouseWheelBehavior::ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos)
{
	FInputRayHit RayHit;
	if (TestShouldRespondToMouseWheelFunc.IsBound())
	{
		RayHit = TestShouldRespondToMouseWheelFunc.Execute(CurrentPos);
	}
	return RayHit;
}

void UScriptableToolMouseWheelBehavior::OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos)
{
	OnMouseWheelScrollUpFunc.ExecuteIfBound(CurrentPos, BehaviorHost->GetActiveModifiers());
}

void UScriptableToolMouseWheelBehavior::OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos)
{
	OnMouseWheelScrollDownFunc.ExecuteIfBound(CurrentPos, BehaviorHost->GetActiveModifiers());
}

void UScriptableToolMouseWheelBehavior::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	BehaviorHost->OnUpdateModifierState(ModifierID, bIsOn);
}
