// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviors/ScriptableToolBehaviorDelegates.h"
#include "ScriptableToolBehavior.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "ScriptableToolMouseWheelBehavior.generated.h"

class UScriptableModularBehaviorTool;
class UMouseWheelInputBehavior;


UCLASS()
class UScriptableToolMouseWheelBehavior : public UScriptableToolBehavior, public IMouseWheelBehaviorTarget
{
	GENERATED_BODY()

public:
	UScriptableToolMouseWheelBehavior() {};

	void Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
		FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
		FTestShouldRespondToMouseWheelDelegate TestShouldRespondToMouseWheelFuncIn,
		FOnMouseWheelScrollUpDelegate OnMouseWheelScrollUpFuncIn,
		FOnMouseWheelScrollDownDelegate OnMouseWheelScrollDownFuncIn);

	virtual UInputBehavior* GetWrappedBehavior() override;

	virtual FInputRayHit ShouldRespondToMouseWheel(const FInputDeviceRay& CurrentPos) override;
	virtual void OnMouseWheelScrollUp(const FInputDeviceRay& CurrentPos) override;
	virtual void OnMouseWheelScrollDown(const FInputDeviceRay& CurrentPos) override;
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

private:

	UPROPERTY()
	TObjectPtr<UScriptableModularBehaviorTool> BehaviorHost;

	UPROPERTY()
	TObjectPtr<UMouseWheelInputBehavior> Behavior;

	UPROPERTY()
	FMouseBehaviorModiferCheckDelegate ModifierCheckFunc;

	UPROPERTY()
	FTestShouldRespondToMouseWheelDelegate	TestShouldRespondToMouseWheelFunc;

	UPROPERTY()
	FOnMouseWheelScrollUpDelegate OnMouseWheelScrollUpFunc;

	UPROPERTY()
	FOnMouseWheelScrollDownDelegate OnMouseWheelScrollDownFunc;

};