// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/ScriptableToolMultiClickSequenceBehavior.h"

#include "BaseTools/ScriptableModularBehaviorTool.h"
#include "BaseBehaviors/MultiClickSequenceInputBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolMultiClickSequenceBehavior)

void UScriptableToolClickSequenceBehavior::Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
	FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
	FMouseBehaviorModiferCheckDelegate HoverModifierCheckFuncIn,
	FOnBeginSequencePreviewDelegate OnBeginSequencePreviewFuncIn,
	FCanBeginClickSequenceDelegate CanBeginClickSequenceFuncIn,
	FOnBeginClickSequenceDelegate OnBeginClickSequenceFuncIn,
	FOnNextSequencePreviewDelegate OnNextSequencePreviewFuncIn,
	FOnNextSequenceClickDelegate OnNextSequenceClickFuncIn,
	FOnTerminateClickSequenceDelegate OnTerminateClickSequenceFuncIn,
	FRequestAbortClickSequenceDelegate RequestAbortClickSequenceFuncIn,
	EScriptableToolMouseButton MouseButtonIn)
{
	BehaviorHost = BehaviorHostIn;
	Behavior = NewObject<UMultiClickSequenceInputBehavior>();
	ModifierCheckFunc = ModifierCheckFuncIn;
	HoverModifierCheckFunc = HoverModifierCheckFuncIn;
	OnBeginSequencePreviewFunc = OnBeginSequencePreviewFuncIn;
	CanBeginClickSequenceFunc = CanBeginClickSequenceFuncIn;
	OnBeginClickSequenceFunc = OnBeginClickSequenceFuncIn;
	OnNextSequencePreviewFunc = OnNextSequencePreviewFuncIn;
	OnNextSequenceClickFunc = OnNextSequenceClickFuncIn;
	OnTerminateClickSequenceFunc = OnTerminateClickSequenceFuncIn;
	RequestAbortClickSequenceFunc = RequestAbortClickSequenceFuncIn;
	MouseButton = MouseButtonIn;

	Behavior->Initialize(this);
	Behavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState) {
		bool bResult = true;
		if (ModifierCheckFunc.IsBound())
		{
			bResult = ModifierCheckFunc.Execute(InputDeviceState);
		}
		return bResult;
	};

	Behavior->HoverModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState) {
		bool bResult = true;
		if (HoverModifierCheckFunc.IsBound())
		{
			bResult = HoverModifierCheckFunc.Execute(InputDeviceState);
		}
		return bResult;
	};

	BehaviorHost->AddInputBehavior(Behavior);

	Behavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	Behavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	Behavior->Modifiers.RegisterModifier(3, FInputDeviceState::IsAltKeyDown);

	switch (MouseButton)
	{
	case EScriptableToolMouseButton::LeftButton:
	{
		Behavior->SetUseLeftMouseButton();
		break;
	}
	case EScriptableToolMouseButton::RightButton:
	{
		Behavior->SetUseRightMouseButton();
		break;
	}
	case EScriptableToolMouseButton::MiddleButton:
	{
		Behavior->SetUseMiddleMouseButton();
		break;
	}
	default:
		ensure(false);
	}
}

UInputBehavior* UScriptableToolClickSequenceBehavior::GetWrappedBehavior()
{
	return Behavior;
}

void UScriptableToolClickSequenceBehavior::OnBeginSequencePreview(const FInputDeviceRay& ClickPos)
{
	OnBeginSequencePreviewFunc.ExecuteIfBound(ClickPos, BehaviorHost->GetActiveModifiers(), MouseButton);
}

bool UScriptableToolClickSequenceBehavior::CanBeginClickSequence(const FInputDeviceRay& ClickPos)
{
	bool bShouldBegin = false;
	if (CanBeginClickSequenceFunc.IsBound())
	{
		bShouldBegin = CanBeginClickSequenceFunc.Execute(ClickPos, MouseButton);
	}
	return bShouldBegin;
}

void UScriptableToolClickSequenceBehavior::OnBeginClickSequence(const FInputDeviceRay& ClickPos)
{
	OnBeginClickSequenceFunc.ExecuteIfBound(ClickPos, BehaviorHost->GetActiveModifiers(), MouseButton);

}

void UScriptableToolClickSequenceBehavior::OnNextSequencePreview(const FInputDeviceRay& ClickPos)
{
	OnNextSequencePreviewFunc.ExecuteIfBound(ClickPos, BehaviorHost->GetActiveModifiers(), MouseButton);
}

bool UScriptableToolClickSequenceBehavior::OnNextSequenceClick(const FInputDeviceRay& ClickPos)
{
	bool bShouldContinue = false;
	if (OnNextSequenceClickFunc.IsBound())
	{
		bShouldContinue = OnNextSequenceClickFunc.Execute(ClickPos, BehaviorHost->GetActiveModifiers(), MouseButton);
	}
	return bShouldContinue;
}

void UScriptableToolClickSequenceBehavior::OnTerminateClickSequence()
{
	OnTerminateClickSequenceFunc.ExecuteIfBound(BehaviorHost->GetActiveModifiers(), MouseButton);
}

bool UScriptableToolClickSequenceBehavior::RequestAbortClickSequence()
{
	bool bShouldTerminate = false;
	if (RequestAbortClickSequenceFunc.IsBound())
	{
		bShouldTerminate = RequestAbortClickSequenceFunc.Execute();
	}
	return bShouldTerminate;
}

void UScriptableToolClickSequenceBehavior::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	BehaviorHost->OnUpdateModifierState(ModifierID, bIsOn);
}
