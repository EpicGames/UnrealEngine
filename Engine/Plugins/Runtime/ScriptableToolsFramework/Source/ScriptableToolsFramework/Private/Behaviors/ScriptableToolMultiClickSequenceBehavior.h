// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviors/ScriptableToolBehaviorDelegates.h"
#include "ScriptableToolBehavior.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "ScriptableToolMultiClickSequenceBehavior.generated.h"

class UScriptableModularBehaviorTool;
class UMultiClickSequenceInputBehavior;


UCLASS()
class UScriptableToolClickSequenceBehavior : public UScriptableToolBehavior, public IClickSequenceBehaviorTarget
{
	GENERATED_BODY()

public:
	UScriptableToolClickSequenceBehavior() {};

	void Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
		FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
		FMouseBehaviorModiferCheckDelegate HoverModifierCheckFuncIn,
		FOnBeginSequencePreviewDelegate OnBeginSequencePreviewFuncIn,
		FCanBeginClickSequenceDelegate CanBeginClickSequenceFuncIn,
		FOnBeginClickSequenceDelegate OnBeginClickSequenceFuncIn,
		FOnNextSequencePreviewDelegate OnNextSequencePreviewFuncIn,
		FOnNextSequenceClickDelegate OnNextSequenceClickFuncIn,
		FOnTerminateClickSequenceDelegate OnTerminateClickSequenceFuncIn,
		FRequestAbortClickSequenceDelegate RequestAbortClickSequenceFuncIn,
		EScriptableToolMouseButton MouseButtonIn);

	virtual UInputBehavior* GetWrappedBehavior() override;

	virtual void OnBeginSequencePreview(const FInputDeviceRay& ClickPos) override;
	virtual bool CanBeginClickSequence(const FInputDeviceRay& ClickPos) override;
	virtual void OnBeginClickSequence(const FInputDeviceRay& ClickPos) override;
	virtual void OnNextSequencePreview(const FInputDeviceRay& ClickPos) override;
	virtual bool OnNextSequenceClick(const FInputDeviceRay& ClickPos) override;
	virtual void OnTerminateClickSequence() override;
	virtual bool RequestAbortClickSequence()  override;
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

private:

	UPROPERTY()
	TObjectPtr<UScriptableModularBehaviorTool> BehaviorHost;

	UPROPERTY()
	TObjectPtr<UMultiClickSequenceInputBehavior> Behavior;

	UPROPERTY()
	FMouseBehaviorModiferCheckDelegate ModifierCheckFunc;

	UPROPERTY()
	FMouseBehaviorModiferCheckDelegate HoverModifierCheckFunc;

	UPROPERTY()
	FOnBeginSequencePreviewDelegate OnBeginSequencePreviewFunc;

	UPROPERTY()
	FCanBeginClickSequenceDelegate CanBeginClickSequenceFunc;

	UPROPERTY()
	FOnBeginClickSequenceDelegate OnBeginClickSequenceFunc;

	UPROPERTY()
	FOnNextSequencePreviewDelegate OnNextSequencePreviewFunc;

	UPROPERTY()
	FOnNextSequenceClickDelegate OnNextSequenceClickFunc;

	UPROPERTY()
	FOnTerminateClickSequenceDelegate OnTerminateClickSequenceFunc;

	UPROPERTY()
	FRequestAbortClickSequenceDelegate RequestAbortClickSequenceFunc;

	UPROPERTY()
	EScriptableToolMouseButton MouseButton;
};