// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/ScriptableToolSingleClickBehavior.h"

#include "BaseTools/ScriptableModularBehaviorTool.h"
#include "BaseBehaviors/SingleClickBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolSingleClickBehavior)

void UScriptableToolSingleClickBehavior::Init(
	TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
	FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
	FTestIfHitByClickDelegate TestIfHitByClickFuncIn,
	FOnHitByClickDelegate OnHitByClickFuncIn,
	EScriptableToolMouseButton MouseButtonIn,
	bool bHitTestOnReleaseIn
	)
{
	BehaviorHost = BehaviorHostIn;
	Behavior = CreateNewBehavior();
	ModifierCheckFunc = ModifierCheckFuncIn;
	TestIfHitByClickFunc = TestIfHitByClickFuncIn;
	OnHitByClickFunc = OnHitByClickFuncIn;
	MouseButton = MouseButtonIn;

	Behavior->Initialize(this);
	Behavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState) {
		bool bResult = true;
		if (ModifierCheckFunc.IsBound())
		{
			bResult = ModifierCheckFunc.Execute(InputDeviceState);
		}
		return bResult;
	};

	Behavior->HitTestOnRelease = bHitTestOnReleaseIn;

	BehaviorHost->AddInputBehavior(Behavior);

	Behavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	Behavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	Behavior->Modifiers.RegisterModifier(3, FInputDeviceState::IsAltKeyDown);

	switch (MouseButton)
	{
	case EScriptableToolMouseButton::LeftButton:
	{
		Behavior->SetUseLeftMouseButton();
		break;
	}
	case EScriptableToolMouseButton::RightButton:
	{
		Behavior->SetUseRightMouseButton();
		break;
	}
	case EScriptableToolMouseButton::MiddleButton:
	{
		Behavior->SetUseMiddleMouseButton();
		break;
	}
	default:
		ensure(false);
	}
}

USingleClickInputBehavior* UScriptableToolSingleClickBehavior::CreateNewBehavior() const
{
	return NewObject<USingleClickInputBehavior>();
}

UInputBehavior* UScriptableToolSingleClickBehavior::GetWrappedBehavior()
{
	return Behavior;
}

FInputRayHit UScriptableToolSingleClickBehavior::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit RayHit;
	if (TestIfHitByClickFunc.IsBound())
	{
		RayHit = TestIfHitByClickFunc.Execute(ClickPos, MouseButton);
	}
	return RayHit;
}

void UScriptableToolSingleClickBehavior::OnClicked(const FInputDeviceRay& ClickPos)
{
	OnHitByClickFunc.ExecuteIfBound(ClickPos, BehaviorHost->GetActiveModifiers(), MouseButton);
}

void UScriptableToolSingleClickBehavior::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	BehaviorHost->OnUpdateModifierState(ModifierID, bIsOn);
}
