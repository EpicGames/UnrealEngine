// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviors/ScriptableToolBehaviorDelegates.h"
#include "ScriptableToolBehavior.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "ScriptableToolSingleClickBehavior.generated.h"

class UScriptableModularBehaviorTool;
class USingleClickInputBehavior;

UCLASS()
class UScriptableToolSingleClickBehavior : public UScriptableToolBehavior, public IClickBehaviorTarget
{
	GENERATED_BODY()

public:
	UScriptableToolSingleClickBehavior() {};

	void Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
		FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
		FTestIfHitByClickDelegate TestIfHitByClickFuncIn,
		FOnHitByClickDelegate OnHitByClickFuncIn,
		EScriptableToolMouseButton MouseButtonIn,
		bool bHitTestOnReleaseIn);

	virtual USingleClickInputBehavior* CreateNewBehavior() const;
	virtual UInputBehavior* GetWrappedBehavior() override;

	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPos) override;
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

private:

	UPROPERTY()
	TObjectPtr<UScriptableModularBehaviorTool> BehaviorHost;

	UPROPERTY()
	TObjectPtr<USingleClickInputBehavior> Behavior;

	UPROPERTY()
	FMouseBehaviorModiferCheckDelegate ModifierCheckFunc;

	UPROPERTY()
	FTestIfHitByClickDelegate	TestIfHitByClickFunc;

	UPROPERTY()
	FOnHitByClickDelegate OnHitByClickFunc;

	UPROPERTY()
	EScriptableToolMouseButton MouseButton;
};
