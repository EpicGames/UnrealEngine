// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/ScriptableToolSingleClickOrDragBehavior.h"

#include "BaseTools/ScriptableModularBehaviorTool.h"
#include "BaseBehaviors/SingleClickOrDragBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolSingleClickOrDragBehavior)


void UScriptableToolSingleClickOrDragBehavior::Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
	FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
	FTestIfHitByClickDelegate TestIfHitByClickFuncIn,
	FOnHitByClickDelegate OnHitByClickFuncIn,
	FTestCanBeginClickDragSequenceDelegate TestCanBeginClickDragSequenceFuncIn,
	FOnClickPressDelegate OnClickPressFuncIn,
	FOnClickDragDelegate OnClickDragFuncIn,
	FOnClickReleaseDelegate OnClickReleaseFuncIn,
	FOnTerminateDragSequenceDelegate OnTerminateDragSequenceFuncIn,
	EScriptableToolMouseButton MouseButtonIn,
	bool bBeginDragIfClickTargetNotHitIn,
	float ClickDistanceThresholdIn)
{
	BehaviorHost = BehaviorHostIn;
	Behavior = NewObject<USingleClickOrDragInputBehavior>();
	ModifierCheckFunc = ModifierCheckFuncIn;
	TestIfHitByClickFunc = TestIfHitByClickFuncIn;
	OnHitByClickFunc = OnHitByClickFuncIn;
	TestCanBeginClickDragSequenceFunc = TestCanBeginClickDragSequenceFuncIn;
	OnClickPressFunc = OnClickPressFuncIn;
	OnClickDragFunc = OnClickDragFuncIn;
	OnClickReleaseFunc = OnClickReleaseFuncIn;
	OnTerminateDragSequenceFunc = OnTerminateDragSequenceFuncIn;
	MouseButton = MouseButtonIn;

	Behavior->Initialize(this, this);
	Behavior->ModifierCheckFunc = [this](const FInputDeviceState& InputDeviceState) {
		bool bResult = true;
		if (ModifierCheckFunc.IsBound())
		{
			bResult = ModifierCheckFunc.Execute(InputDeviceState);
		}
		return bResult;
	};

	Behavior->bBeginDragIfClickTargetNotHit = bBeginDragIfClickTargetNotHitIn;
	Behavior->ClickDistanceThreshold = ClickDistanceThresholdIn;

	BehaviorHost->AddInputBehavior(Behavior);

	Behavior->Modifiers.RegisterModifier(1, FInputDeviceState::IsShiftKeyDown);
	Behavior->Modifiers.RegisterModifier(2, FInputDeviceState::IsCtrlKeyDown);
	Behavior->Modifiers.RegisterModifier(3, FInputDeviceState::IsAltKeyDown);

	switch (MouseButton)
	{
	case EScriptableToolMouseButton::LeftButton:
	{
		Behavior->SetUseLeftMouseButton();
		break;
	}
	case EScriptableToolMouseButton::RightButton:
	{
		Behavior->SetUseRightMouseButton();
		break;
	}
	case EScriptableToolMouseButton::MiddleButton:
	{
		Behavior->SetUseMiddleMouseButton();
		break;
	}
	default:
		ensure(false);
	}
}

UInputBehavior* UScriptableToolSingleClickOrDragBehavior::GetWrappedBehavior()
{
	return Behavior;
}

FInputRayHit UScriptableToolSingleClickOrDragBehavior::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit RayHit;
	if (TestIfHitByClickFunc.IsBound())
	{
		RayHit = TestIfHitByClickFunc.Execute(ClickPos, MouseButton);
	}
	return RayHit;
}

void UScriptableToolSingleClickOrDragBehavior::OnClicked(const FInputDeviceRay& ClickPos)
{
	OnHitByClickFunc.ExecuteIfBound(ClickPos, BehaviorHost->GetActiveModifiers(), MouseButton);
}

FInputRayHit UScriptableToolSingleClickOrDragBehavior::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	FInputRayHit RayHit;
	if (TestCanBeginClickDragSequenceFunc.IsBound())
	{
		RayHit = TestCanBeginClickDragSequenceFunc.Execute(PressPos, BehaviorHost->GetActiveModifiers(), MouseButton);
	}
	return RayHit;
}

void UScriptableToolSingleClickOrDragBehavior::OnClickPress(const FInputDeviceRay& PressPos)
{
	OnClickPressFunc.ExecuteIfBound(PressPos, BehaviorHost->GetActiveModifiers(), MouseButton);
}

void UScriptableToolSingleClickOrDragBehavior::OnClickDrag(const FInputDeviceRay& DragPos)
{
	OnClickDragFunc.ExecuteIfBound(DragPos, BehaviorHost->GetActiveModifiers(), MouseButton);
}

void UScriptableToolSingleClickOrDragBehavior::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	OnClickReleaseFunc.ExecuteIfBound(ReleasePos, BehaviorHost->GetActiveModifiers(), MouseButton);
}

void UScriptableToolSingleClickOrDragBehavior::OnTerminateDragSequence()
{
	OnTerminateDragSequenceFunc.ExecuteIfBound(BehaviorHost->GetActiveModifiers(), MouseButton);
}

void UScriptableToolSingleClickOrDragBehavior::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	BehaviorHost->OnUpdateModifierState(ModifierID, bIsOn);
}
