// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviors/ScriptableToolBehaviorDelegates.h"
#include "ScriptableToolBehavior.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "ScriptableToolSingleClickOrDragBehavior.generated.h"

class UScriptableModularBehaviorTool;
class USingleClickOrDragInputBehavior;


UCLASS()
class UScriptableToolSingleClickOrDragBehavior : public UScriptableToolBehavior, public IClickDragBehaviorTarget, public IClickBehaviorTarget
{
	GENERATED_BODY()

public:
	UScriptableToolSingleClickOrDragBehavior() {};

	void Init(TObjectPtr<UScriptableModularBehaviorTool> BehaviorHostIn,
		FMouseBehaviorModiferCheckDelegate ModifierCheckFuncIn,
		FTestIfHitByClickDelegate TestIfHitByClickFuncIn,
		FOnHitByClickDelegate OnHitByClickFuncIn,
		FTestCanBeginClickDragSequenceDelegate TestCanBeginClickDragSequenceFuncIn,
		FOnClickPressDelegate OnClickPressFuncIn,
		FOnClickDragDelegate OnClickDragFuncIn,
		FOnClickReleaseDelegate OnClickReleaseFuncIn,
		FOnTerminateDragSequenceDelegate OnTerminateDragSequenceFuncIn,
		EScriptableToolMouseButton MouseButtonIn,
		bool bBeginDragIfClickTargetNotHitIn,
		float ClickDistanceThresholdIn);

	virtual UInputBehavior* GetWrappedBehavior() override;

	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	virtual void OnClicked(const FInputDeviceRay& ClickPos) override;
	virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	virtual void OnTerminateDragSequence() override;
	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

private:

	UPROPERTY()
	TObjectPtr<UScriptableModularBehaviorTool> BehaviorHost;

	UPROPERTY()
	TObjectPtr<USingleClickOrDragInputBehavior> Behavior;

	UPROPERTY()
	FMouseBehaviorModiferCheckDelegate ModifierCheckFunc;

	UPROPERTY()
	FTestIfHitByClickDelegate	TestIfHitByClickFunc;

	UPROPERTY()
	FOnHitByClickDelegate OnHitByClickFunc;

	UPROPERTY()
	FTestCanBeginClickDragSequenceDelegate	TestCanBeginClickDragSequenceFunc;

	UPROPERTY()
	FOnClickPressDelegate OnClickPressFunc;

	UPROPERTY()
	FOnClickDragDelegate OnClickDragFunc;

	UPROPERTY()
	FOnClickReleaseDelegate OnClickReleaseFunc;

	UPROPERTY()
	FOnTerminateDragSequenceDelegate OnTerminateDragSequenceFunc;

	UPROPERTY()
	EScriptableToolMouseButton MouseButton;
};