// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/ScriptableToolLine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolLine)


void UScriptableToolLine::SetLineID(int32 LineIDIn)
{
	LineID = LineIDIn;
}

int32 UScriptableToolLine::GetLineID() const
{
	return LineID;
}

bool UScriptableToolLine::IsDirty() const
{
	return bIsDirty;
}

FRenderableLine UScriptableToolLine::GenerateLineDescription()
{
	bIsDirty = false;
	return LineDescription;
}

void UScriptableToolLine::SetLineStart(FVector LineStartIn)
{
	LineDescription.Start = LineStartIn;
	bIsDirty = true;
}

void UScriptableToolLine::SetLineEnd(FVector LineEndIn)
{
	LineDescription.End = LineEndIn;
	bIsDirty = true;
}

void UScriptableToolLine::SetLineEndPoints(FVector LineStartIn, FVector LineEndIn)
{
	LineDescription.Start = LineStartIn;
	LineDescription.End = LineEndIn;
	bIsDirty = true;
}

void UScriptableToolLine::SetLineColor(FColor LineColorIn)
{
	LineDescription.Color = LineColorIn;
	bIsDirty = true;
}

void UScriptableToolLine::SetLineThickness(float LineThicknessIn)
{
	LineDescription.Thickness = LineThicknessIn;
	bIsDirty = true;
}

void UScriptableToolLine::SetLineDepthBias(float LineDepthBiasIn)
{
	LineDescription.DepthBias = LineDepthBiasIn;
	bIsDirty = true;
}
