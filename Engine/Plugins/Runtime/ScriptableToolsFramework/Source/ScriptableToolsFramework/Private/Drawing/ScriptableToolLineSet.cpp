// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/ScriptableToolLineSet.h"

#include "Drawing/ScriptableToolLine.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Misc/Guid.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolLineSet)

void UScriptableToolLineSet::Initialize(TObjectPtr<UPreviewGeometry> PreviewGeometry)
{
	FString LineSetID = FGuid::NewGuid().ToString();
	WeakLineSet = PreviewGeometry->AddLineSet(LineSetID);
}

void UScriptableToolLineSet::OnTick()
{
	if (ULineSetComponent* ResolvedLineSet = WeakLineSet.Get())
	{
		for (TObjectPtr<UScriptableToolLine> LineComponent : LineComponents)
		{
			if (LineComponent->IsDirty())
			{
				int32 LineID = LineComponent->GetLineID();
				FRenderableLine LineDescription = LineComponent->GenerateLineDescription();

				ResolvedLineSet->SetLineStart(LineID, LineDescription.Start);
				ResolvedLineSet->SetLineEnd(LineID, LineDescription.End);
				ResolvedLineSet->SetLineColor(LineID, LineDescription.Color);
				ResolvedLineSet->SetLineThickness(LineID, LineDescription.Thickness);
			}
		}
	}
}

UScriptableToolLine* UScriptableToolLineSet::AddLine()
{
	UScriptableToolLine* NewLine = nullptr;
	if (ULineSetComponent* ResolvedLineSet = WeakLineSet.Get())
	{
		LineComponents.Add(NewObject<UScriptableToolLine>(this));
		NewLine = LineComponents.Last();
		NewLine->SetLineID(ResolvedLineSet->AddLine(NewLine->GenerateLineDescription()));
	}
	return NewLine;
}

void UScriptableToolLineSet::RemoveLine(UScriptableToolLine* Line)
{
	if (ULineSetComponent* ResolvedLineSet = WeakLineSet.Get())
	{
		if (ensure(Line))
		{
			ResolvedLineSet->RemoveLine(Line->GetLineID());
			LineComponents.Remove(Line);
		}
	}
}

void UScriptableToolLineSet::RemoveAllLines()
{
	if (ULineSetComponent* ResolvedLineSet = WeakLineSet.Get())
	{
		ResolvedLineSet->Clear();
	}
	LineComponents.Empty();
}

void UScriptableToolLineSet::SetAllLinesColor(FColor Color)
{
	if (ULineSetComponent* ResolvedLineSet = WeakLineSet.Get())
	{
		ResolvedLineSet->SetAllLinesColor(Color);
	}
}

void UScriptableToolLineSet::SetAllLinesThickness(float Thickness)
{
	if (ULineSetComponent* ResolvedLineSet = WeakLineSet.Get())
	{
		ResolvedLineSet->SetAllLinesThickness(Thickness);
	}
}
