// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/ScriptableToolPoint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolPoint)


void UScriptableToolPoint::SetPointID(int32 PointIDIn)
{
	PointID = PointIDIn;
}

int32 UScriptableToolPoint::GetPointID() const
{
	return PointID;
}

bool UScriptableToolPoint::IsDirty() const
{
	return bIsDirty;
}

FRenderablePoint UScriptableToolPoint::GeneratePointDescription()
{
	bIsDirty = false;
	return PointDescription;
}

void UScriptableToolPoint::SetPointPosition(FVector PositionIn)
{
	PointDescription.Position = PositionIn;
	bIsDirty = true;
}

void UScriptableToolPoint::SetPointColor(FColor PointColorIn)
{
	PointDescription.Color = PointColorIn;
	bIsDirty = true;
}

void UScriptableToolPoint::SetPointSize(float SizeIn)
{
	PointDescription.Size = SizeIn;
	bIsDirty = true;
}

void UScriptableToolPoint::SetPointDepthBias(float PointDepthBiasIn)
{
	PointDescription.DepthBias = PointDepthBiasIn;
	bIsDirty = true;
}
