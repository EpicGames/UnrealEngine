// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/ScriptableToolPointSet.h"

#include "Drawing/ScriptableToolPoint.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Misc/Guid.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolPointSet)

void UScriptableToolPointSet::Initialize(TObjectPtr<UPreviewGeometry> PreviewGeometry)
{
	FString PointSetID = FGuid::NewGuid().ToString();
	WeakPointSet = PreviewGeometry->AddPointSet(PointSetID);
}

void UScriptableToolPointSet::OnTick()
{
	if (UPointSetComponent* ResolvedPointSet = WeakPointSet.Get())
	{
		for (TObjectPtr<UScriptableToolPoint> PointComponent : PointComponents)
		{
			if (PointComponent->IsDirty())
			{
				int32 PointID = PointComponent->GetPointID();
				FRenderablePoint PointDescription = PointComponent->GeneratePointDescription();

				ResolvedPointSet->SetPointPosition(PointID, PointDescription.Position);			
				ResolvedPointSet->SetPointColor(PointID, PointDescription.Color);
				ResolvedPointSet->SetPointSize(PointID, PointDescription.Size);
			}
		}
	}
}

UScriptableToolPoint* UScriptableToolPointSet::AddPoint()
{
	UScriptableToolPoint* NewPoint = nullptr;
	if (UPointSetComponent* ResolvedPointSet = WeakPointSet.Get())
	{
		PointComponents.Add(NewObject<UScriptableToolPoint>(this));
		NewPoint = PointComponents.Last();
		NewPoint->SetPointID(ResolvedPointSet->AddPoint(NewPoint->GeneratePointDescription()));
	}
	return NewPoint;
}

void UScriptableToolPointSet::RemovePoint(UScriptableToolPoint* Point)
{
	if (UPointSetComponent* ResolvedPointSet = WeakPointSet.Get())
	{
		if (ensure(Point))
		{
			ResolvedPointSet->RemovePoint(Point->GetPointID());
			PointComponents.Remove(Point);
		}
	}
}

void UScriptableToolPointSet::RemoveAllPoints()
{
	if (UPointSetComponent* ResolvedPointSet = WeakPointSet.Get())
	{
		ResolvedPointSet->Clear();
	}
	PointComponents.Empty();
}

void UScriptableToolPointSet::SetAllPointsColor(FColor Color)
{
	if (UPointSetComponent* ResolvedPointSet = WeakPointSet.Get())
	{
		ResolvedPointSet->SetAllPointsColor(Color);
	}
}

void UScriptableToolPointSet::SetAllPointsSize(float Size)
{
	if (UPointSetComponent* ResolvedPointSet = WeakPointSet.Get())
	{
		ResolvedPointSet->SetAllPointsSize(Size);
	}
}
