// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/ScriptableToolTriangle.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolTriangle)

UScriptableToolTriangle::UScriptableToolTriangle()
{
	TriangleDescription.Material = UMaterial::GetDefaultMaterial(EMaterialDomain::MD_Surface);
}

void UScriptableToolTriangle::SetTriangleID(int32 TriangleIDIn)
{
	TriangleID = TriangleIDIn;
}

int32 UScriptableToolTriangle::GetTriangleID() const
{
	return TriangleID;
}

bool UScriptableToolTriangle::IsDirty() const
{
	return bIsDirty;
}

FRenderableTriangle UScriptableToolTriangle::GenerateTriangleDescription()
{
	bIsDirty = false;
	return TriangleDescription;
}

void UScriptableToolTriangle::SetTriangleMaterial(UMaterialInterface* Material)
{
	TriangleDescription.Material = Material;
	bIsDirty = true;
}

void UScriptableToolTriangle::SetTrianglePoints(FVector A, FVector B, FVector C)
{
	TriangleDescription.Vertex0.Position = A;
	TriangleDescription.Vertex1.Position = B;
	TriangleDescription.Vertex2.Position = C;
	bIsDirty = true;
}

void UScriptableToolTriangle::SetTriangleUVs(FVector2D A, FVector2D B, FVector2D C)
{
	TriangleDescription.Vertex0.UV = A;
	TriangleDescription.Vertex1.UV = B;
	TriangleDescription.Vertex2.UV = C;
	bIsDirty = true;
}

void UScriptableToolTriangle::SetTriangleNormals(FVector A, FVector B, FVector C)
{
	TriangleDescription.Vertex0.Normal = A;
	TriangleDescription.Vertex1.Normal = B;
	TriangleDescription.Vertex2.Normal = C;
	bIsDirty = true;
}

void UScriptableToolTriangle::SetTriangleColors(FColor A, FColor B, FColor C)
{
	TriangleDescription.Vertex0.Color = A;
	TriangleDescription.Vertex1.Color = B;
	TriangleDescription.Vertex2.Color = C;
	bIsDirty = true;
}




UScriptableToolQuad::UScriptableToolQuad()
{
	TriangleADescription.Material = UMaterial::GetDefaultMaterial(EMaterialDomain::MD_Surface);
	TriangleBDescription.Material = UMaterial::GetDefaultMaterial(EMaterialDomain::MD_Surface);
}


void UScriptableToolQuad::SetTriangleAID(int32 TriangleIDIn)
{
	TriangleAID = TriangleIDIn;
}

int32 UScriptableToolQuad::GetTriangleAID() const
{
	return TriangleAID;
}

void UScriptableToolQuad::SetTriangleBID(int32 TriangleIDIn)
{
	TriangleBID = TriangleIDIn;
}

int32 UScriptableToolQuad::GetTriangleBID() const
{
	return TriangleBID;
}

bool UScriptableToolQuad::IsDirty() const
{
	return bIsDirty;
}

TPair<FRenderableTriangle, FRenderableTriangle> UScriptableToolQuad::GenerateQuadDescription()
{
	bIsDirty = false;
	return TPair<FRenderableTriangle, FRenderableTriangle>(TriangleADescription, TriangleBDescription);
}

void UScriptableToolQuad::SetQuadMaterial(UMaterialInterface* Material)
{
	TriangleADescription.Material = Material;
	TriangleBDescription.Material = Material;
	bIsDirty = true;
}

void UScriptableToolQuad::SetQuadPoints(FVector A, FVector B, FVector C, FVector D)
{
	TriangleADescription.Vertex0.Position = TriangleBDescription.Vertex0.Position = A;
	TriangleADescription.Vertex1.Position = B;
	TriangleADescription.Vertex2.Position = TriangleBDescription.Vertex1.Position = C;
	TriangleBDescription.Vertex2.Position = D;
	bIsDirty = true;
}

void UScriptableToolQuad::SetQuadUVs(FVector2D A, FVector2D B, FVector2D C, FVector2D D)
{
	TriangleADescription.Vertex0.UV = TriangleBDescription.Vertex0.UV = A;
	TriangleADescription.Vertex1.UV = B;
	TriangleADescription.Vertex2.UV = TriangleBDescription.Vertex1.UV = C;
	TriangleBDescription.Vertex2.UV = D;
	bIsDirty = true;
}

void UScriptableToolQuad::SetQuadNormals(FVector A, FVector B, FVector C, FVector D)
{
	TriangleADescription.Vertex0.Normal = TriangleBDescription.Vertex0.Normal = A;
	TriangleADescription.Vertex1.Normal = B;
	TriangleADescription.Vertex2.Normal = TriangleBDescription.Vertex1.Normal = C;
	TriangleBDescription.Vertex2.Normal = D;
	bIsDirty = true;
}

void UScriptableToolQuad::SetQuadColors(FColor A, FColor B, FColor C, FColor D)
{
	TriangleADescription.Vertex0.Color = TriangleBDescription.Vertex0.Color = A;
	TriangleADescription.Vertex1.Color = B;
	TriangleADescription.Vertex2.Color = TriangleBDescription.Vertex1.Color = C;
	TriangleBDescription.Vertex2.Color = D;
	bIsDirty = true;
}
