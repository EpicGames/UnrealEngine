// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/ScriptableToolTriangleSet.h"

#include "Drawing/ScriptableToolTriangle.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Misc/Guid.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolTriangleSet)

void UScriptableToolTriangleSet::Initialize(TObjectPtr<UPreviewGeometry> PreviewGeometry)
{
	FString TriangleSetID = FGuid::NewGuid().ToString();
	WeakTriangleSet = PreviewGeometry->AddTriangleSet(TriangleSetID);
}

void UScriptableToolTriangleSet::OnTick()
{
	if (UTriangleSetComponent* ResolvedTriangleSet = WeakTriangleSet.Get())
	{
		for (TObjectPtr<UScriptableToolTriangle> TriangleComponent : TriangleComponents)
		{
			if (TriangleComponent->IsDirty())
			{
				int32 TriangleID = TriangleComponent->GetTriangleID();
				FRenderableTriangle TriangleDescription = TriangleComponent->GenerateTriangleDescription();

				ResolvedTriangleSet->RemoveTriangle(TriangleID);
				ResolvedTriangleSet->InsertTriangle(TriangleID, TriangleDescription);
			}
		}

		for (TObjectPtr<UScriptableToolQuad> QuadComponent : QuadComponents)
		{
			if (QuadComponent->IsDirty())
			{
				int32 TriangleAID = QuadComponent->GetTriangleAID();
				int32 TriangleBID = QuadComponent->GetTriangleBID();

				TPair<FRenderableTriangle, FRenderableTriangle> TriangleDescriptions = QuadComponent->GenerateQuadDescription();

				ResolvedTriangleSet->RemoveTriangle(TriangleAID);
				ResolvedTriangleSet->InsertTriangle(TriangleAID, TriangleDescriptions.Key);
				ResolvedTriangleSet->RemoveTriangle(TriangleBID);
				ResolvedTriangleSet->InsertTriangle(TriangleBID, TriangleDescriptions.Value);

			}
		}
	}
}

UScriptableToolTriangle* UScriptableToolTriangleSet::AddTriangle()
{
	UScriptableToolTriangle* NewTriangle = nullptr;
	if (UTriangleSetComponent* ResolvedTriangleSet = WeakTriangleSet.Get())
	{
		TriangleComponents.Add(NewObject<UScriptableToolTriangle>(this));
		NewTriangle = TriangleComponents.Last();
		NewTriangle->SetTriangleID(ResolvedTriangleSet->AddTriangle(NewTriangle->GenerateTriangleDescription()));
	}
	return NewTriangle;
}

UScriptableToolQuad* UScriptableToolTriangleSet::AddQuad()
{
	UScriptableToolQuad* NewQuad = nullptr;
	if (UTriangleSetComponent* ResolvedTriangleSet = WeakTriangleSet.Get())
	{
		QuadComponents.Add(NewObject<UScriptableToolQuad>(this));
		NewQuad = QuadComponents.Last();

		TPair<FRenderableTriangle, FRenderableTriangle> TriDescriptions = NewQuad->GenerateQuadDescription();

		int32 TriAID = ResolvedTriangleSet->AddTriangle(TriDescriptions.Key);
		int32 TriBID = ResolvedTriangleSet->AddTriangle(TriDescriptions.Value);

		NewQuad->SetTriangleAID(TriAID);
		NewQuad->SetTriangleBID(TriBID);
	}
	return NewQuad;
}

void UScriptableToolTriangleSet::RemoveTriangle(UScriptableToolTriangle* Triangle)
{
	if (UTriangleSetComponent* ResolvedTriangleSet = WeakTriangleSet.Get())
	{
		if (ensure(Triangle))
		{
			ResolvedTriangleSet->RemoveTriangle(Triangle->GetTriangleID());
			TriangleComponents.Remove(Triangle);
		}
	}
}

void UScriptableToolTriangleSet::RemoveQuad(UScriptableToolQuad* Quad)
{
	if (UTriangleSetComponent* ResolvedTriangleSet = WeakTriangleSet.Get())
	{
		if (ensure(Quad))
		{
			ResolvedTriangleSet->RemoveTriangle(Quad->GetTriangleAID());
			ResolvedTriangleSet->RemoveTriangle(Quad->GetTriangleBID());
			QuadComponents.Remove(Quad);
		}
	}
}


void UScriptableToolTriangleSet::RemoveAllFaces()
{
	if (UTriangleSetComponent* ResolvedTriangleSet = WeakTriangleSet.Get())
	{
		ResolvedTriangleSet->Clear();
	}
	TriangleComponents.Empty();
}

void UScriptableToolTriangleSet::SetAllTrianglesColor(FColor Color)
{
	if (UTriangleSetComponent* ResolvedTriangleSet = WeakTriangleSet.Get())
	{
		ResolvedTriangleSet->SetAllTrianglesColor(Color);
	}
}

void UScriptableToolTriangleSet::SetAllTrianglesMaterial(UMaterialInterface* Material)
{
	if (UTriangleSetComponent* ResolvedTriangleSet = WeakTriangleSet.Get())
	{
		ResolvedTriangleSet->SetAllTrianglesMaterial(Material);
	}
}
