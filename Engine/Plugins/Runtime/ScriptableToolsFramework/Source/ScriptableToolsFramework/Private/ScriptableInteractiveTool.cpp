// Copyright Epic Games, Inc. All Rights Reserved.


#include "ScriptableInteractiveTool.h"
#include "BaseGizmos/TransformProxy.h"
#include "InteractiveToolManager.h"

#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/CombinedTransformGizmo.h"

#include "Drawing/PreviewGeometryActor.h"
#include "Drawing/ScriptableToolLineSet.h"
#include "Drawing/ScriptableToolPointSet.h"
#include "Drawing/ScriptableToolTriangleSet.h"


#include "Engine/Font.h"
#include "ToolDataVisualizer.h"
#include "CanvasTypes.h"
#include "Engine/Engine.h"  // for GEngine->GetSmallFont()
#include "InteractiveGizmoManager.h"
#include "SceneView.h"

#include "Blueprint/UserWidget.h"
#include "ContextObjectStore.h"
#include "Utility/ScriptableToolContextObjects.h"
#include "Slate/SObjectWidget.h"
#include "Components/Widget.h"


#include "UObject/EnumProperty.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableInteractiveTool)

#define LOCTEXT_NAMESPACE "UScriptableInteractiveTool"

DECLARE_LOG_CATEGORY_CLASS(LogScriptableInteractiveTool, Log, All);

namespace ScriptableInteractiveToolLocals
{
	class SScriptableToolViewportOverlayDragContainer : public SObjectWidget
	{ };
}
namespace UELocal
{

	template<typename PropertyType>
	PropertyType* FindValidPropertyByTypeAndName(
		UScriptableInteractiveToolPropertySet* PropertySet,
		FString PropertyName,
		FString CallerString)
	{
		if (PropertySet == nullptr)
		{
			UE_LOGF(LogTemp, Warning, "%ls: Property Set is null", *CallerString);
			return nullptr;
		}

		for (TFieldIterator<FProperty> PropIt(PropertySet->GetClass(), EFieldIterationFlags::None); PropIt; ++PropIt)
		{
			FProperty* Property = *PropIt;
			if (Property->GetName() == PropertyName)
			{
				if (PropertyType* TypedProp = CastField<PropertyType>(*PropIt))
				{
					return TypedProp;
				}
				else
				{
					UE_LOGF(LogTemp, Warning, "%ls: Property %ls does not have a compatible Type", *CallerString, *PropertyName);
					return nullptr;
				}
			}
		}

		UE_LOGF(LogTemp, Warning, "%ls: Could not find a Property named %ls in the Property Set %ls", *CallerString, *PropertyName,
			*PropertySet->GetName());
		return nullptr;
	}

} // end namespace UELocal


UScriptableInteractiveToolPropertySet* UScriptableInteractiveToolPropertySet::SetPropertyAsReadOnly(FString PropertyName, bool bReadOnly)
{
	FProperty* Property = UELocal::FindValidPropertyByTypeAndName<FProperty>(
		this, PropertyName, TEXT("SetReadOnly"));
	if (Property == nullptr)
	{
		return this;
	}
	if (!bReadOnly)
	{
		Property->ClearPropertyFlags(EPropertyFlags::CPF_EditConst);
	}
	else
	{
		Property->SetPropertyFlags(EPropertyFlags::CPF_EditConst);
	}

	if (UScriptableInteractiveTool* OwningTool = ParentTool.Get())
	{
		OwningTool->OnPropertyModifiedDirectlyByTool.Broadcast(this);
	}

	return this;
}

UScriptableInteractiveToolPropertySet* UScriptableInteractiveToolPropertySet::SetPropertyAsHidden(FString PropertyName, bool bHidden)
{
	FProperty* Property = UELocal::FindValidPropertyByTypeAndName<FProperty>(
		this, PropertyName, TEXT("SetHidden"));
	if (Property == nullptr)
	{
		return this;
	}
	if (bHidden)
	{
		Property->ClearPropertyFlags(EPropertyFlags::CPF_Edit);
	}
	else
	{
		Property->SetPropertyFlags(EPropertyFlags::CPF_Edit);
	}

	if (UScriptableInteractiveTool* OwningTool = ParentTool.Get())
	{
		OwningTool->OnPropertyModifiedDirectlyByTool.Broadcast(this);
	}

	return this;
}

UScriptableInteractiveToolPropertySet* UScriptableInteractiveToolPropertySet::MakeTransactional()
{
	SetFlags(RF_Transactional);
	return this;
}

UScriptableInteractiveTool* UScriptableInteractiveToolPropertySet::GetOwningTool(EToolsFrameworkOutcomePins& Outcome)
{
	if (UScriptableInteractiveTool* OwningTool = ParentTool.Get())
	{
		Outcome = EToolsFrameworkOutcomePins::Success;
		return OwningTool;
	}
	Outcome = EToolsFrameworkOutcomePins::Failure;
	return nullptr;
}

UWorld* UScriptableInteractiveToolPropertySet::GetWorld() const
{
	UWorld* Result = nullptr;
	if (UScriptableInteractiveTool* OwningTool = ParentTool.Get())
	{
		Result = OwningTool->GetWorld();
	}
	return Result;
}

UScriptableTool_RenderAPI* UScriptableTool_RenderAPI::DrawLine(FVector Start, FVector End, FLinearColor Color, float Thickness, float DepthBias, bool bDepthTested)
{
	if (ActiveVisualizer == nullptr)
	{
		UE_LOGF(LogTemp, Warning, "[ScriptableInteractiveTool::DrawLine] DrawLine can only be called during OnScriptRender event callback");
		return this;
	}

	ActiveVisualizer->DepthBias = DepthBias;
	ActiveVisualizer->DrawLine(Start, End, Color, Thickness, bDepthTested);
	return this;
}


UScriptableTool_RenderAPI* UScriptableTool_RenderAPI::DrawRectWidthHeightXY(FTransform Transform, double Width, double Height, FLinearColor Color, float LineThickness, float DepthBias, bool bDepthTested, bool bOriginIsCenter)
{
	if (ActiveVisualizer == nullptr)
	{
		UE_LOGF(LogTemp, Warning, "[ScriptableInteractiveTool::DrawRectWidthHeight] DrawRectWidthHeight can only be called during OnScriptRender event callback");
		return this;
	}

	ActiveVisualizer->DepthBias = DepthBias;

	ActiveVisualizer->PushTransform(Transform);

	if (bOriginIsCenter)
	{
		FVector DeltaX(Width*0.5, 0, 0), DeltaY(0,Height*0.5,0);
		FVector A(-DeltaX-DeltaY), B(DeltaX-DeltaY), C(DeltaX+DeltaY), D(-DeltaX+DeltaY);
		ActiveVisualizer->DrawLine(A, B, Color, LineThickness, bDepthTested);
		ActiveVisualizer->DrawLine(B, C, Color, LineThickness, bDepthTested);
		ActiveVisualizer->DrawLine(C, D, Color, LineThickness, bDepthTested);
		ActiveVisualizer->DrawLine(D, A, Color, LineThickness, bDepthTested);
	}
	else
	{
		FVector DeltaX(Width, 0, 0), DeltaY(0,Height,0);
		FVector A(0,0,0), B(DeltaX), C(DeltaX+DeltaY), D(DeltaY);
		ActiveVisualizer->DrawLine(A, B, Color, LineThickness, bDepthTested);
		ActiveVisualizer->DrawLine(B, C, Color, LineThickness, bDepthTested);
		ActiveVisualizer->DrawLine(C, D, Color, LineThickness, bDepthTested);
		ActiveVisualizer->DrawLine(D, A, Color, LineThickness, bDepthTested);
	}

	ActiveVisualizer->PopTransform();

	return this;
}



UScriptableTool_HUDAPI* UScriptableTool_HUDAPI::DrawTextAtLocation(FVector Location, FString String, FLinearColor Color, bool bCentered, float ShiftRowsY)
{
	if (ActiveCanvas == nullptr || ActiveSceneView == nullptr)
	{
		UE_LOGF(LogTemp, Warning, "[ScriptableInteractiveTool::DrawTextAtLocation] DrawTextAtLocation can only be called during OnScriptDrawHUD event callback");
		return this;
	}

	float DPIScale = ActiveCanvas->GetDPIScale();
	UFont* UseFont = GEngine->GetSmallFont();

	FVector2D PixelPos;
	ActiveSceneView->WorldToPixel(Location, PixelPos);

	double Height = (double)UseFont->GetStringHeightSize(TEXT("yjMT"));
	PixelPos.Y -= Height;
	PixelPos.Y -= ShiftRowsY * Height;

	if ( bCentered )
	{
		float StringWidth = UseFont->GetStringSize(*String);
		PixelPos.X -= 0.5*StringWidth;
	}

	ActiveCanvas->DrawShadowedString(PixelPos.X / (double)DPIScale, PixelPos.Y / (double)DPIScale, *String, UseFont, Color);

	return this;
}


UScriptableTool_HUDAPI* UScriptableTool_HUDAPI::DrawTextArrayAtLocation(FVector Location, TArray<FString> Strings, FLinearColor Color, bool bCentered, float ShiftRowsY)
{
	if (ActiveCanvas == nullptr || ActiveSceneView == nullptr)
	{
		UE_LOGF(LogTemp, Warning, "[ScriptableInteractiveTool::DrawTextAtLocation] DrawTextAtLocation can only be called during OnScriptDrawHUD event callback");
		return this;
	}

	float DPIScale = ActiveCanvas->GetDPIScale();
	UFont* UseFont = GEngine->GetSmallFont();

	FVector2D PixelPos;
	ActiveSceneView->WorldToPixel(Location, PixelPos);

	double Height = (double)UseFont->GetStringHeightSize(TEXT("yjMT"));
	double CurY = PixelPos.Y;
	CurY -= Height;
	CurY -= ShiftRowsY * Height;

	int32 NumStrings = Strings.Num();
	if (NumStrings > 1)
	{
		Height *= 1.5;
	}

	for (int32 k = 0; k < NumStrings; ++k)
	{
		const FString& CurString = Strings[NumStrings-k-1];
		
		double ShiftX = 0;
		if (bCentered)
		{
			ShiftX = -0.5 * UseFont->GetStringSize(*CurString);
		}

		ActiveCanvas->DrawShadowedString((PixelPos.X + ShiftX) / (double)DPIScale, CurY / (double)DPIScale, *CurString, UseFont, Color);
		CurY -= Height;
	}

	return this;
}


UScriptableTool_HUDAPI* UScriptableTool_HUDAPI::GetCanvasLocation(FVector Location, FVector2D& CanvasLocation)
{
	if (ActiveCanvas == nullptr || ActiveSceneView == nullptr)
	{
		UE_LOGF(LogTemp, Warning, "[ScriptableInteractiveTool::GetCanvasLocation] DrawTextAtLocation can only be called during OnScriptDrawHUD event callback");
		CanvasLocation = FVector2D::Zero();
	}
	else
	{
		float DPIScale = ActiveCanvas->GetDPIScale();
		FVector2D PixelPos;
		ActiveSceneView->WorldToPixel(Location, PixelPos);
		CanvasLocation = PixelPos / (double)DPIScale;
	}
	return this;
}



void UScriptableInteractiveTool::Setup()
{
	UInteractiveTool::Setup();

	RenderHelper = NewObject<UScriptableTool_RenderAPI>();
	DrawHUDHelper = NewObject<UScriptableTool_HUDAPI>();

	ToolDrawableGeometry = NewObject<UPreviewGeometry>();
	ToolDrawableGeometry->CreateInWorld(GetWorld(), FTransform::Identity);

	DefaultLineSet = NewObject<UScriptableToolLineSet>();
	DefaultLineSet->Initialize(ToolDrawableGeometry);

	DefaultPointSet = NewObject<UScriptableToolPointSet>();
	DefaultPointSet->Initialize(ToolDrawableGeometry);

	DefaultTriangleSet = NewObject<UScriptableToolTriangleSet>();
	DefaultTriangleSet->Initialize(ToolDrawableGeometry);

	OnScriptSetup();
}

void UScriptableInteractiveTool::OnTick(float DeltaTime)
{
	UInteractiveTool::OnTick(DeltaTime);
	OnScriptTick(DeltaTime);

	DefaultLineSet->OnTick();
	for (UScriptableToolLineSet* LineSet : LineSets)
	{
		LineSet->OnTick();
	}

	DefaultPointSet->OnTick();
	for (UScriptableToolPointSet* PointSet : PointSets)
	{
		PointSet->OnTick();
	}

	DefaultTriangleSet->OnTick();
	for (UScriptableToolTriangleSet* TriangleSet : TriangleSets)
	{
		TriangleSet->OnTick();
	}
}

bool UScriptableInteractiveTool::HasAccept() const
{
	return (ToolShutdownType == EScriptableToolShutdownType::AcceptCancel);
}
bool UScriptableInteractiveTool::HasCancel() const
{
	return (ToolShutdownType == EScriptableToolShutdownType::AcceptCancel);
}

bool UScriptableInteractiveTool::CanAccept() const 
{
	if (ToolShutdownType != EScriptableToolShutdownType::AcceptCancel)
	{
		return false;
	}
	return OnScriptCanAccept();
}

bool UScriptableInteractiveTool::OnScriptCanAccept_Implementation() const
{
	return HasAccept();
}


void UScriptableInteractiveTool::Shutdown(EToolShutdownType ShutdownType)
{
	OnScriptShutdown(ShutdownType);

	// clean up any gizmos
	for (TPair<FString, TObjectPtr<UInteractiveGizmo>> Gizmo : Gizmos)
	{
		GetToolManager()->GetPairedGizmoManager()->DestroyGizmo(Gizmo.Value);
	}
	Gizmos.Reset();

	RenderHelper = nullptr;
	DrawHUDHelper = nullptr;

	ToolDrawableGeometry->Disconnect();
	ToolDrawableGeometry = nullptr;

	UInteractiveTool::Shutdown(ShutdownType);
}

void UScriptableInteractiveTool::RequestToolShutdown(
	bool bAccept,
	bool bShowUserPopupMessage,
	FText UserMessage)
{
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		EToolShutdownType UseShutdownType = EToolShutdownType::Completed;
		if (ToolShutdownType == EScriptableToolShutdownType::AcceptCancel)
		{
			UseShutdownType = (bAccept) ? EToolShutdownType::Accept : EToolShutdownType::Cancel;
		}
		ToolManager->PostActiveToolShutdownRequest(this, UseShutdownType, bShowUserPopupMessage, UserMessage);
	}
}



void UScriptableInteractiveTool::PostInitProperties()
{
	Super::PostInitProperties();

	// because this was done in the C++ Constructor, it will be wiped out when a BP Subclass
	// initializes itself from it's CDO
	if (InputBehaviors == nullptr)
	{
		InputBehaviors = NewObject<UInputBehaviorSet>(this, TEXT("InputBehaviors"));
	}
}


UBaseScriptableToolBuilder* UScriptableInteractiveTool::GetNewCustomToolBuilderInstance(UObject* Outer)
{
	switch (ToolStartupRequirements)
	{
	case EScriptableToolStartupRequirements::None:
		return NewObject<UBaseScriptableToolBuilder>(Outer);
	case EScriptableToolStartupRequirements::ToolTarget:
	{
		if (!ToolTargetToolBuilderClass->IsValidLowLevelFast())
		{
			return NewObject<UBaseScriptableToolBuilder>(Outer);
		}

		TObjectPtr<UCustomScriptableToolBuilderContainer> BuilderWrapper = NewObject<UCustomScriptableToolBuilderContainer>(Outer);
		TObjectPtr<UCustomScriptableToolBuilderComponentBase> InterfacePointer = NewObject<UCustomScriptableToolBuilderComponentBase>(Outer, ToolTargetToolBuilderClass.Get());
		Cast<UToolTargetScriptableToolBuilder>(InterfacePointer)->Initialize();
		BuilderWrapper->Initialize(InterfacePointer);

		return BuilderWrapper;
	}	
	case EScriptableToolStartupRequirements::Custom:
	{
		if (!CustomToolBuilderClass->IsValidLowLevelFast())
		{
			return NewObject<UBaseScriptableToolBuilder>(Outer);
		}

		TObjectPtr<UCustomScriptableToolBuilderContainer> BuilderWrapper = NewObject<UCustomScriptableToolBuilderContainer>(Outer);
		TObjectPtr<UCustomScriptableToolBuilderComponentBase> InterfacePointer = NewObject<UCustomScriptableToolBuilderComponentBase>(Outer, CustomToolBuilderClass.Get());
		BuilderWrapper->Initialize(InterfacePointer);

		return BuilderWrapper;
	}
	default:
		ensure(false);
		return NewObject<UBaseScriptableToolBuilder>(Outer);
	}
}


void UScriptableInteractiveTool::SetTargetWorld(UWorld* World)
{
	TargetWorld = World;
}

UWorld* UScriptableInteractiveTool::GetWorld() const 
{
	return TargetWorld.Get() ; 
}


UWorld* UScriptableInteractiveTool::GetToolWorld()
{
	return TargetWorld.Get();
}




void UScriptableInteractiveTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	FToolDataVisualizer Renderer;
	Renderer.BeginFrame(RenderAPI);

	RenderHelper->ActiveVisualizer = &Renderer;

	OnScriptRender(RenderHelper);

	RenderHelper->ActiveVisualizer = nullptr;

	Renderer.EndFrame();
}
void UScriptableInteractiveTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	DrawHUDHelper->ActiveCanvas = Canvas;
	DrawHUDHelper->ActiveSceneView = RenderAPI->GetSceneView();

	OnScriptDrawHUD(DrawHUDHelper);

	DrawHUDHelper->ActiveCanvas = nullptr;
	DrawHUDHelper->ActiveSceneView = nullptr;

}


UScriptableInteractiveToolPropertySet* UScriptableInteractiveTool::AddPropertySetOfType(
	TSubclassOf<UScriptableInteractiveToolPropertySet> PropertySetType,
	FString Identifier,
	EToolsFrameworkOutcomePins& Outcome)
{
	Outcome = EToolsFrameworkOutcomePins::Failure;

	if (NamedPropertySets.Contains(Identifier) || Identifier.Len() == 0 )
	{
		UE_LOGF(LogTemp, Warning, "[AddPropertySetOfType] PropertySet Identifier %ls is invalid or already in use!", *Identifier);
		return nullptr;
	}

	UClass* Class = PropertySetType.Get();
	UClass* BasePropertySetClass = UScriptableInteractiveToolPropertySet::StaticClass();
	if (Class->IsChildOf(BasePropertySetClass) == false)
	{
		UE_LOGF(LogTemp, Warning, "[AddPropertySetOfType] Class Type %ls is not a UScriptableInteractiveToolPropertySet Subclass!", *Class->GetAuthoredName());
		return nullptr;
	}

	UObject* NewPropertySetObj = NewObject<UScriptableInteractiveToolPropertySet>(this, Class);
	if (NewPropertySetObj == nullptr)
	{
		UE_LOGF(LogTemp, Warning, "[AddPropertySetOfType] failed to create instance of Class Type %ls", *Class->GetAuthoredName());
		return nullptr;
	}
	UScriptableInteractiveToolPropertySet* NewPropertySet = Cast<UScriptableInteractiveToolPropertySet>(NewPropertySetObj);

	// ensure that this property set is marked Transient, as it should never be saved
	NewPropertySet->SetFlags(RF_Transient);

	NewPropertySet->ParentTool = this;

	AddToolPropertySource(NewPropertySet);
	NamedPropertySets.Add(Identifier, NewPropertySet);

	Outcome = EToolsFrameworkOutcomePins::Success;
	return NewPropertySet;
}


void UScriptableInteractiveTool::RemovePropertySetByName(
	FString Identifier,
	EToolsFrameworkOutcomePins& Outcome)
{
	Outcome = EToolsFrameworkOutcomePins::Failure;

	TWeakObjectPtr<UScriptableInteractiveToolPropertySet>* Found = NamedPropertySets.Find(Identifier);
	if (Found != nullptr)
	{
		NamedPropertySets.Remove(Identifier);
		if ((*Found).IsValid())
		{
			bool bRemoved = RemoveToolPropertySource( (*Found).Get() );
			ensure(bRemoved == true);
		}
		Outcome = EToolsFrameworkOutcomePins::Success;
	}
}

void UScriptableInteractiveTool::SetPropertySetVisibleByName(
	FString Identifier,
	bool bVisible)
{
	TWeakObjectPtr<UScriptableInteractiveToolPropertySet>* Found = NamedPropertySets.Find(Identifier);
	if (Found != nullptr && (*Found).IsValid() )
	{
		SetToolPropertySourceEnabled( (*Found).Get(), bVisible );
	}
}

void UScriptableInteractiveTool::ForcePropertySetUpdateByName(
	FString Identifier)
{
	TWeakObjectPtr<UScriptableInteractiveToolPropertySet>* Found = NamedPropertySets.Find(Identifier);
	if (Found != nullptr && (*Found).IsValid())
	{
		NotifyOfPropertyChangeByTool((*Found).Get());
	}
}


UScriptableInteractiveToolPropertySet* UScriptableInteractiveTool::RestorePropertySetSettings(
	UScriptableInteractiveToolPropertySet* PropertySet,
	FString SaveKey)
{
	if (PropertySet)
	{
		PropertySet->RestoreProperties(this, SaveKey);
	}
	return PropertySet;
}


UScriptableInteractiveToolPropertySet* UScriptableInteractiveTool::SavePropertySetSettings(
	UScriptableInteractiveToolPropertySet* PropertySet,
	FString SaveKey)
{
	if (PropertySet)
	{
		PropertySet->SaveProperties(this, SaveKey);
	}
	return PropertySet;
}







UScriptableInteractiveToolPropertySet* UScriptableInteractiveTool::WatchFloatProperty(
	UScriptableInteractiveToolPropertySet* PropertySet,
	FString PropertyName,
	const FToolFloatPropertyModifiedDelegate& OnModified)
{
	if (FNumericProperty* NumericProp = UELocal::FindValidPropertyByTypeAndName<FNumericProperty>(
			PropertySet, PropertyName, TEXT("WatchFloatProperty")))
	{
		if (NumericProp->IsFloatingPoint() == false)
		{
			UE_LOGF(LogTemp, Warning, "WatchFloatProperty: Property %ls is not Float type", *PropertyName);
			return PropertySet;
		}

		int32 ArrayIndex = WatchFloatPropertyDelegates.Num();
		WatchFloatPropertyDelegates.Add(OnModified);

		PropertySet->WatchProperty<float>(
			[NumericProp, PropertySet]()->float { 
				// is this value stable? could it be captured instead?
				const void* ValuePtr = NumericProp->ContainerPtrToValuePtr<void>(PropertySet);
				return (double)NumericProp->GetFloatingPointPropertyValue(ValuePtr);
			},
			[this, PropertySet, PropertyName, ArrayIndex](double NewValue) { 
				WatchFloatPropertyDelegates[ArrayIndex].ExecuteIfBound(PropertySet, PropertyName, NewValue);
			} );
	}
	return PropertySet;
}



UScriptableInteractiveToolPropertySet* UScriptableInteractiveTool::WatchIntProperty(
	UScriptableInteractiveToolPropertySet* PropertySet,
	FString PropertyName,
	const FToolIntPropertyModifiedDelegate& OnModified)
{
	if (FNumericProperty* NumericProp = UELocal::FindValidPropertyByTypeAndName<FNumericProperty>(
			PropertySet, PropertyName, TEXT("WatchIntProperty")))
	{
		if (NumericProp->IsInteger() == false)
		{
			UE_LOGF(LogTemp, Warning, "WatchIntProperty: Property %ls is not Integer type", *PropertyName);
			return PropertySet;
		}

		int32 ArrayIndex = WatchIntPropertyDelegates.Num();
		WatchIntPropertyDelegates.Add(OnModified);

		PropertySet->WatchProperty<int>(
			[NumericProp, PropertySet]()->int { 
				const void* ValuePtr = NumericProp->ContainerPtrToValuePtr<void>(PropertySet);
				return (int)NumericProp->GetSignedIntPropertyValue(ValuePtr);
			},
			[this, PropertySet, PropertyName, ArrayIndex](int NewValue) { 
				WatchIntPropertyDelegates[ArrayIndex].ExecuteIfBound(PropertySet, PropertyName, NewValue);
			} );
	}
	return PropertySet;
}



UScriptableInteractiveToolPropertySet* UScriptableInteractiveTool::WatchBoolProperty(
	UScriptableInteractiveToolPropertySet* PropertySet,
	FString PropertyName,
	const FToolBoolPropertyModifiedDelegate& OnModified)
{
	if (FBoolProperty* BoolProp = UELocal::FindValidPropertyByTypeAndName<FBoolProperty>(
			PropertySet, PropertyName, TEXT("WatchBoolProperty")))
	{
		int32 ArrayIndex = WatchBoolPropertyDelegates.Num();
		WatchBoolPropertyDelegates.Add(OnModified);

		PropertySet->WatchProperty<bool>(
			[BoolProp, PropertySet]()->bool { 
				const void* ValuePtr = BoolProp->ContainerPtrToValuePtr<void>(PropertySet);
				bool bValue = BoolProp->GetPropertyValue(ValuePtr);
				return bValue;
			},
			[this, PropertySet, PropertyName, ArrayIndex](bool bNewValue) { 
				WatchBoolPropertyDelegates[ArrayIndex].ExecuteIfBound(PropertySet, PropertyName, bNewValue);
			} );
	}
	return PropertySet;
}


UScriptableInteractiveToolPropertySet* UScriptableInteractiveTool::WatchEnumProperty(
	UScriptableInteractiveToolPropertySet* PropertySet,
	FString PropertyName,
	const FToolEnumPropertyModifiedDelegate& OnModified)
{
	if (FEnumProperty* EnumProperty = UELocal::FindValidPropertyByTypeAndName<FEnumProperty>(
			PropertySet, PropertyName, TEXT("WatchEnumProperty")))
	{
		FNumericProperty* EnumIntProperty = EnumProperty->GetUnderlyingProperty();
		int32 ArrayIndex = WatchEnumPropertyDelegates.Num();
		WatchEnumPropertyDelegates.Add(OnModified);

		PropertySet->WatchProperty<int>(
			[EnumIntProperty, EnumProperty, PropertySet]()->int64 { 
				const void* ValuePtr = EnumProperty->ContainerPtrToValuePtr<void>(PropertySet);
				int64 Value = EnumIntProperty->GetSignedIntPropertyValue(ValuePtr);
				return Value;
			},
			[this, PropertySet, PropertyName, ArrayIndex](int64 NewValue) { 
				WatchEnumPropertyDelegates[ArrayIndex].ExecuteIfBound(PropertySet, PropertyName, (uint8)NewValue);
			} );
	} else if (FByteProperty* ByteProperty = UELocal::FindValidPropertyByTypeAndName<FByteProperty>(
		PropertySet, PropertyName, TEXT("WatchEnumProperty")))
	{
		int32 ArrayIndex = WatchEnumPropertyDelegates.Num();
		WatchEnumPropertyDelegates.Add(OnModified);

		PropertySet->WatchProperty<int>(
			[ByteProperty, PropertySet]()->int64 {
				const void* ValuePtr = ByteProperty->ContainerPtrToValuePtr<void>(PropertySet);
				int64 Value = ByteProperty->GetSignedIntPropertyValue(ValuePtr);
				return Value;
			},
			[this, PropertySet, PropertyName, ArrayIndex](int64 NewValue) {
				WatchEnumPropertyDelegates[ArrayIndex].ExecuteIfBound(PropertySet, PropertyName, (uint8)NewValue);
			});
	}

	return PropertySet;
}



UScriptableInteractiveToolPropertySet* UScriptableInteractiveTool::WatchStringProperty(
	UScriptableInteractiveToolPropertySet* PropertySet,
	FString PropertyName,
	const FToolStringPropertyModifiedDelegate& OnModified)
{
	if (FStrProperty* StringProp = UELocal::FindValidPropertyByTypeAndName<FStrProperty>(
			PropertySet, PropertyName, TEXT("WatchStringProperty")))
	{
		int32 ArrayIndex = WatchStringPropertyDelegates.Num();
		WatchStringPropertyDelegates.Add(OnModified);

		PropertySet->WatchProperty<FString>(
			[StringProp, PropertySet]()->FString { 
				const void* ValuePtr = StringProp->ContainerPtrToValuePtr<void>(PropertySet);
				return StringProp->GetPropertyValue(ValuePtr);
			},
			[this, PropertySet, PropertyName, ArrayIndex](FString NewValue) { 
				WatchStringPropertyDelegates[ArrayIndex].ExecuteIfBound(PropertySet, PropertyName, NewValue);
			} );
	}
	return PropertySet;
}


UScriptableInteractiveToolPropertySet* UScriptableInteractiveTool::WatchNameProperty(
	UScriptableInteractiveToolPropertySet* PropertySet,
	FString PropertyName,
	const FToolFNamePropertyModifiedDelegate& OnModified)
{
	if (FNameProperty* NameProp = UELocal::FindValidPropertyByTypeAndName<FNameProperty>(
			PropertySet, PropertyName, TEXT("WatchNameProperty")))
	{
		int32 ArrayIndex = WatchFNamePropertyDelegates.Num();
		WatchFNamePropertyDelegates.Add(OnModified);

		PropertySet->WatchProperty<FName>(
			[NameProp, PropertySet]()->FName { 
				const void* ValuePtr = NameProp->ContainerPtrToValuePtr<void>(PropertySet);
				return NameProp->GetPropertyValue(ValuePtr);
			},
			[this, PropertySet, PropertyName, ArrayIndex](FName NewValue) { 
				WatchFNamePropertyDelegates[ArrayIndex].ExecuteIfBound(PropertySet, PropertyName, NewValue);
			} );
	}
	return PropertySet;
}


UScriptableInteractiveToolPropertySet* UScriptableInteractiveTool::WatchObjectProperty(
	UScriptableInteractiveToolPropertySet* PropertySet,
	FString PropertyName,
	const FToolObjectPropertyModifiedDelegate& OnModified)
{
	if (FObjectPropertyBase* ObjectProp = UELocal::FindValidPropertyByTypeAndName<FObjectPropertyBase>(
			PropertySet, PropertyName, TEXT("WatchObjectProperty")))
	{
		int32 ArrayIndex = WatchObjectPropertyDelegates.Num();
		WatchObjectPropertyDelegates.Add(OnModified);

		PropertySet->WatchProperty<UObject*>(
			[ObjectProp, PropertySet]() { 
				const void* ValuePtr = ObjectProp->ContainerPtrToValuePtr<void>(PropertySet);
				return ObjectProp->GetObjectPropertyValue(ValuePtr);
			},
			[this, PropertySet, PropertyName, ArrayIndex](UObject* NewValue) { 
				WatchObjectPropertyDelegates[ArrayIndex].ExecuteIfBound(PropertySet, PropertyName, NewValue);
			} );
	}
	return PropertySet;
}




UScriptableInteractiveToolPropertySet* UScriptableInteractiveTool::WatchProperty(
	UScriptableInteractiveToolPropertySet* PropertySet,
	FString PropertyName,
	const FToolPropertyModifiedDelegate& OnModified)
{
	FProperty* Property = UELocal::FindValidPropertyByTypeAndName<FProperty>(
			PropertySet, PropertyName, TEXT("WatchProperty"));
	if (Property == nullptr)
	{
		return PropertySet;
	}

	int32 ArrayIndex = WatchAnyPropertyInfo.Num();
	FAnyPropertyWatchInfo NewWatchInfo;
	NewWatchInfo.Delegate = OnModified;
	WatchAnyPropertyInfo.Add(NewWatchInfo);

	// FMapProperty, FSetProperty, FDelegateProperty, FInterfaceProperty

	if (FNumericProperty* NumericProp = CastField<FNumericProperty>(Property))
	{
		if (NumericProp->IsInteger())
		{
			WatchAnyPropertyInfo[ArrayIndex].KnownType = EAnyPropertyWatchTypes::Integer;
			PropertySet->WatchProperty<int>(
				[NumericProp, PropertySet]()->int { 
					const void* ValuePtr = NumericProp->ContainerPtrToValuePtr<void>(PropertySet);
					return (int)NumericProp->GetSignedIntPropertyValue(ValuePtr);
				},
				[this, PropertySet, PropertyName, ArrayIndex](int NewValue) { 
					WatchAnyPropertyInfo[ArrayIndex].Delegate.ExecuteIfBound(PropertySet, PropertyName);
				} );
		} 
		else if (NumericProp->IsFloatingPoint())
		{
			WatchAnyPropertyInfo[ArrayIndex].KnownType = EAnyPropertyWatchTypes::Double;
			PropertySet->WatchProperty<float>(
				[NumericProp, PropertySet]()->float { 
					// is this value stable? could it be captured instead?
					const void* ValuePtr = NumericProp->ContainerPtrToValuePtr<void>(PropertySet);
					return (double)NumericProp->GetFloatingPointPropertyValue(ValuePtr);
				},
				[this, PropertySet, PropertyName, ArrayIndex](double NewValue) { 
					WatchAnyPropertyInfo[ArrayIndex].Delegate.ExecuteIfBound(PropertySet, PropertyName);
				} );
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "WatchProperty: Property %ls is an unsupported type", *PropertyName);
		}
	}
	else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		WatchAnyPropertyInfo[ArrayIndex].KnownType = EAnyPropertyWatchTypes::Enum;
		FNumericProperty* EnumIntProperty = EnumProperty->GetUnderlyingProperty();
		PropertySet->WatchProperty<int>(
			[EnumIntProperty, EnumProperty, PropertySet]()->int64 { 
				const void* ValuePtr = EnumProperty->ContainerPtrToValuePtr<void>(PropertySet);
				return (int)EnumIntProperty->GetSignedIntPropertyValue(ValuePtr);
			},
			[this, PropertySet, PropertyName, ArrayIndex](int64 NewValue) { 
				WatchAnyPropertyInfo[ArrayIndex].Delegate.ExecuteIfBound(PropertySet, PropertyName);
			} );
	}
	else if (FBoolProperty* BoolProp = CastField<FBoolProperty>(Property))
	{
		WatchAnyPropertyInfo[ArrayIndex].KnownType = EAnyPropertyWatchTypes::Bool;
		PropertySet->WatchProperty<bool>(
			[BoolProp, PropertySet]()->bool { 
				const void* ValuePtr = BoolProp->ContainerPtrToValuePtr<void>(PropertySet);
				return BoolProp->GetPropertyValue(ValuePtr);
			},
			[this, PropertySet, PropertyName, ArrayIndex](bool bNewValue) { 
				WatchAnyPropertyInfo[ArrayIndex].Delegate.ExecuteIfBound(PropertySet, PropertyName);
			} );
	}
	else if (FStrProperty* StringProperty = CastField<FStrProperty>(Property))
	{
		WatchAnyPropertyInfo[ArrayIndex].KnownType = EAnyPropertyWatchTypes::String;
		PropertySet->WatchProperty<FString>(
			[StringProperty, PropertySet]()->FString { 
				const void* ValuePtr = StringProperty->ContainerPtrToValuePtr<void>(PropertySet);
				return StringProperty->GetPropertyValue(ValuePtr);
			},
			[this, PropertySet, PropertyName, ArrayIndex](FString NewValue) { 
				WatchAnyPropertyInfo[ArrayIndex].Delegate.ExecuteIfBound(PropertySet, PropertyName);
			} );				
	}
	else if (FNameProperty* NameProp = CastField<FNameProperty>(Property))
	{
		WatchAnyPropertyInfo[ArrayIndex].KnownType = EAnyPropertyWatchTypes::FName;
		PropertySet->WatchProperty<FName>(
			[NameProp, PropertySet]()->FName { 
				const void* ValuePtr = NameProp->ContainerPtrToValuePtr<void>(PropertySet);
				return NameProp->GetPropertyValue(ValuePtr);
			},
			[this, PropertySet, PropertyName, ArrayIndex](FName NewValue) { 
				WatchAnyPropertyInfo[ArrayIndex].Delegate.ExecuteIfBound(PropertySet, PropertyName);
			} );
	}
	else if (FObjectPropertyBase* BaseObjProperty = CastField<FObjectPropertyBase>(Property) )
	{
		WatchAnyPropertyInfo[ArrayIndex].KnownType = EAnyPropertyWatchTypes::Object;
		PropertySet->WatchProperty<UObject*>(
			[BaseObjProperty, PropertySet]() { 
				const void* ValuePtr = BaseObjProperty->ContainerPtrToValuePtr<void>(PropertySet);
				return BaseObjProperty->GetObjectPropertyValue(ValuePtr);
			},
			[this, PropertySet, PropertyName, ArrayIndex](UObject* NewValue) { 
				WatchAnyPropertyInfo[ArrayIndex].Delegate.ExecuteIfBound(PropertySet, PropertyName);
			} );
	}
	else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		WatchAnyPropertyInfo[ArrayIndex].KnownType = EAnyPropertyWatchTypes::Struct;
		PropertySet->WatchProperty<uint32>(
			[this, ArrayIndex, StructProperty, PropertySet]()->uint32 { 
				const void* StructValuePtr = StructProperty->ContainerPtrToValuePtr<void>(PropertySet);
				uint32 ValueHash = StructProperty->GetValueTypeHash(StructValuePtr);
				return ValueHash;
			},
			[this, PropertySet, PropertyName, ArrayIndex](uint32 NewValue) { 
				WatchAnyPropertyInfo[ArrayIndex].Delegate.ExecuteIfBound(PropertySet, PropertyName);
			} );
	}
	else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		WatchAnyPropertyInfo[ArrayIndex].KnownType = EAnyPropertyWatchTypes::Array;
		PropertySet->WatchProperty<uint32>(
			[this, ArrayIndex, ArrayProperty, PropertySet]()->uint32 { 
				const void* ArrayValuePtr = ArrayProperty->ContainerPtrToValuePtr<void>(PropertySet);
				FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayValuePtr);
				int32 ElementSize = ArrayProperty->Inner->GetElementSize();
				int32 NumElements = ArrayHelper.Num();
 				uint32 CRCValue = NumElements;
				for ( int32 k = 0; k < NumElements; ++k )
				{
					CRCValue = FCrc::MemCrc32(ArrayHelper.GetRawPtr(k), ElementSize, CRCValue);
				}
				return CRCValue;
			},
			[this, PropertySet, PropertyName, ArrayIndex](uint32 NewValue) { 
				WatchAnyPropertyInfo[ArrayIndex].Delegate.ExecuteIfBound(PropertySet, PropertyName);
			} );
	}
	else
	{
		UE_LOGF(LogTemp, Warning, "WatchProperty: Property Type for Property %ls is not currently supported", *PropertyName)

		//PropertySet->WatchProperty<uint32>(
		//	[this, ArrayIndex, Property, PropertySet]()->uint32 { 
		//		TArray<uint8>& UseTempBuffer = WatchAnyPropertyInfo[ArrayIndex].TempBuffer;
		//		if (UseTempBuffer.Num() != Property->GetSize())
		//		{
		//			UseTempBuffer.SetNum(Property->GetSize(), false);
		//		}
		//		const void* ValuePtr = Property->ContainerPtrToValuePtr<void>(PropertySet);
		//		Property->CopyCompleteValue(UseTempBuffer.GetData(), ValuePtr);
		//		uint32 CRCValue = FCrc::MemCrc32(UseTempBuffer.GetData(), UseTempBuffer.Num());
		//		return CRCValue;
		//	},
		//	[this, PropertySet, PropertyName, ArrayIndex](uint32 NewValue) { 
		//		WatchAnyPropertyInfo[ArrayIndex].Delegate.ExecuteIfBound(PropertySet, PropertyName);
		//	} );
	}
	return PropertySet;
}




void UScriptableInteractiveTool::CreateTRSGizmo(
	FString Identifier,
	FTransform InitialTransform,
	FScriptableToolGizmoOptions GizmoOptions,
	EToolsFrameworkOutcomePins& Outcome)
{
	Outcome = EToolsFrameworkOutcomePins::Failure;

	if (Gizmos.Contains(Identifier) || Identifier.Len() == 0)
	{
		UE_LOGF(LogTemp, Warning, "[CreateTRSGizmo] Gizmo Identifier %ls is invalid or already in use!", *Identifier);
		return;
	}

	ETransformGizmoSubElements SubElements = ETransformGizmoSubElements::None;
	auto ConfigureGizmoTranslation = [&SubElements, &GizmoOptions](EScriptableToolGizmoTranslation SettingFlag, ETransformGizmoSubElements GizmoFlag)
	{
		EnumAddFlags(SubElements, ((GizmoOptions.TranslationParts & (int32)SettingFlag) != 0) ? GizmoFlag : ETransformGizmoSubElements::None);
	};
	ConfigureGizmoTranslation(EScriptableToolGizmoTranslation::TranslateAxisX, ETransformGizmoSubElements::TranslateAxisX);
	ConfigureGizmoTranslation(EScriptableToolGizmoTranslation::TranslateAxisY, ETransformGizmoSubElements::TranslateAxisY);
	ConfigureGizmoTranslation(EScriptableToolGizmoTranslation::TranslateAxisZ, ETransformGizmoSubElements::TranslateAxisZ);
	ConfigureGizmoTranslation(EScriptableToolGizmoTranslation::TranslatePlaneXY, ETransformGizmoSubElements::TranslatePlaneXY);
	ConfigureGizmoTranslation(EScriptableToolGizmoTranslation::TranslatePlaneXZ, ETransformGizmoSubElements::TranslatePlaneXZ);
	ConfigureGizmoTranslation(EScriptableToolGizmoTranslation::TranslatePlaneYZ, ETransformGizmoSubElements::TranslatePlaneYZ);
	ConfigureGizmoTranslation(EScriptableToolGizmoTranslation::FreeTranslate, ETransformGizmoSubElements::FreeTranslate);

	auto ConfigureGizmoRotation = [&SubElements, &GizmoOptions](EScriptableToolGizmoRotation SettingFlag, ETransformGizmoSubElements GizmoFlag)
	{
		EnumAddFlags(SubElements, ((GizmoOptions.RotationParts & (int32)SettingFlag) != 0) ? GizmoFlag : ETransformGizmoSubElements::None);
	};
	ConfigureGizmoRotation(EScriptableToolGizmoRotation::RotateAxisX, ETransformGizmoSubElements::RotateAxisX);
	ConfigureGizmoRotation(EScriptableToolGizmoRotation::RotateAxisY, ETransformGizmoSubElements::RotateAxisY);
	ConfigureGizmoRotation(EScriptableToolGizmoRotation::RotateAxisZ, ETransformGizmoSubElements::RotateAxisZ);
	ConfigureGizmoRotation(EScriptableToolGizmoRotation::FreeRotate, ETransformGizmoSubElements::FreeRotate);

	auto ConfigureGizmoScale = [&SubElements, &GizmoOptions](EScriptableToolGizmoScale SettingFlag, ETransformGizmoSubElements GizmoFlag)
	{
		EnumAddFlags(SubElements, ((GizmoOptions.ScaleParts & (int32)SettingFlag) != 0) ? GizmoFlag : ETransformGizmoSubElements::None);
	};
	ConfigureGizmoScale(EScriptableToolGizmoScale::ScaleAxisX, ETransformGizmoSubElements::ScaleAxisX);
	ConfigureGizmoScale(EScriptableToolGizmoScale::ScaleAxisY, ETransformGizmoSubElements::ScaleAxisY);
	ConfigureGizmoScale(EScriptableToolGizmoScale::ScaleAxisZ, ETransformGizmoSubElements::ScaleAxisZ);
	ConfigureGizmoScale(EScriptableToolGizmoScale::ScalePlaneXY, ETransformGizmoSubElements::ScalePlaneXY);
	ConfigureGizmoScale(EScriptableToolGizmoScale::ScalePlaneXZ, ETransformGizmoSubElements::ScalePlaneXZ);
	ConfigureGizmoScale(EScriptableToolGizmoScale::ScalePlaneYZ, ETransformGizmoSubElements::ScalePlaneYZ);
	ConfigureGizmoScale(EScriptableToolGizmoScale::ScaleUniform, ETransformGizmoSubElements::ScaleUniform);

	if (SubElements == ETransformGizmoSubElements::None)
	{
		UE_LOGF(LogTemp, Warning, "[CreateTRSGizmo] Cannot create a Transform Gizmo thath as no sub-widgets");
		return;
	}

	UCombinedTransformGizmo* NewGizmo = ( GizmoOptions.bRepositionable ) ?
		UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(GetToolManager(), SubElements, this, Identifier) :
		UE::TransformGizmoUtil::CreateCustomTransformGizmo(GetToolManager(), SubElements, this, Identifier);
	if (NewGizmo == nullptr)
	{
		UE_LOGF(LogTemp, Warning, "[CreateTRSGizmo] Gizmo creation failed");
		return;
	}

	NewGizmo->bUseContextGizmoMode = (GizmoOptions.GizmoMode == EScriptableToolGizmoMode::FromViewportSettings);
	switch (GizmoOptions.GizmoMode)
	{
		case EScriptableToolGizmoMode::Combined:
			NewGizmo->ActiveGizmoMode = EToolContextTransformGizmoMode::Combined;
			break;
		case EScriptableToolGizmoMode::TranslationOnly:
			NewGizmo->ActiveGizmoMode = EToolContextTransformGizmoMode::Translation;
			break;
		case EScriptableToolGizmoMode::RotationOnly:
			NewGizmo->ActiveGizmoMode = EToolContextTransformGizmoMode::Rotation;
			break;
		case EScriptableToolGizmoMode::ScaleOnly:
			NewGizmo->ActiveGizmoMode = EToolContextTransformGizmoMode::Scale;
			break;
	}

	NewGizmo->bUseContextCoordinateSystem = (GizmoOptions.CoordSystem == EScriptableToolGizmoCoordinateSystem::FromViewportSettings);
	switch (GizmoOptions.CoordSystem)
	{
		case EScriptableToolGizmoCoordinateSystem::Local:
			NewGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;
			break;
		case EScriptableToolGizmoCoordinateSystem::World:
			NewGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::World;
			break;
	}

	NewGizmo->bSnapToWorldGrid = GizmoOptions.bSnapTranslation;
	NewGizmo->bSnapToWorldRotGrid = GizmoOptions.bSnapRotation;


	UTransformProxy* GizmoProxy = NewObject<UTransformProxy>(this);
	GizmoProxy->SetTransform(InitialTransform);
	NewGizmo->SetActiveTarget(GizmoProxy, GetToolManager());		// do we always want to route through ToolManager here?

	GizmoProxy->OnTransformChanged.AddWeakLambda(this, [this, Identifier](UTransformProxy*, FTransform NewTransform) 
	{
		OnGizmoTransformChanged_Handler(Identifier, NewTransform);
	});
	GizmoProxy->OnBeginTransformEdit.AddWeakLambda(this, [this, Identifier](UTransformProxy* Proxy) 
	{
		OnGizmoTransformStateChange_Handler(Identifier, Proxy->GetTransform(), EScriptableToolGizmoStateChangeType::BeginTransform);
	});
	GizmoProxy->OnEndTransformEdit.AddWeakLambda(this, [this, Identifier](UTransformProxy* Proxy)
	{
		OnGizmoTransformStateChange_Handler(Identifier, Proxy->GetTransform(), EScriptableToolGizmoStateChangeType::EndTransform);
	});
	GizmoProxy->OnTransformChangedUndoRedo.AddWeakLambda(this, [this, Identifier](UTransformProxy*, FTransform NewTransform)
	{
		OnGizmoTransformStateChange_Handler(Identifier, NewTransform, EScriptableToolGizmoStateChangeType::UndoRedo);
	});
	

	if (GizmoOptions.bAllowNegativeScaling == false)
	{
		NewGizmo->SetDisallowNegativeScaling(true);
	}

	Gizmos.Add(Identifier, NewGizmo);


	Outcome = EToolsFrameworkOutcomePins::Success;
}


void UScriptableInteractiveTool::DestroyTRSGizmo(
	FString Identifier,
	EToolsFrameworkOutcomePins& Outcome)
{
	Outcome = EToolsFrameworkOutcomePins::Failure;

	TObjectPtr<UCombinedTransformGizmo>* Found = Gizmos.Find(Identifier);
	if ( Found == nullptr )
	{
		UE_LOGF(LogTemp, Warning, "[DestroyTRSGizmo] Gizmo Identifier %ls could not be found", *Identifier);
		return;
	}

	if ((*Found)->ActiveTarget)
	{
		(*Found)->ActiveTarget->OnTransformChanged.RemoveAll(this);
		(*Found)->ActiveTarget->OnBeginTransformEdit.RemoveAll(this);
		(*Found)->ActiveTarget->OnEndTransformEdit.RemoveAll(this);
		(*Found)->ActiveTarget->OnTransformChangedUndoRedo.RemoveAll(this);
	}

	GetToolManager()->GetPairedGizmoManager()->DestroyGizmo(*Found);
	Gizmos.Remove(Identifier);
	Outcome = EToolsFrameworkOutcomePins::Success;
}



void UScriptableInteractiveTool::SetGizmoVisible(
	FString Identifier,
	bool bVisible)
{
	TObjectPtr<UCombinedTransformGizmo>* Found = Gizmos.Find(Identifier);
	if (Found != nullptr)
	{
		(*Found)->SetVisibility(bVisible);
	}
	else
	{
		UE_LOGF(LogTemp, Warning, "[SetGizmoVisible] Gizmo Identifier %ls could not be found", *Identifier);
	}
}



void UScriptableInteractiveTool::SetGizmoTransform(
	FString Identifier,
	FTransform NewTransform,
	bool bUndoable)
{
	TObjectPtr<UCombinedTransformGizmo>* Found = Gizmos.Find(Identifier);
	if (Found != nullptr)
	{
		if (bUndoable)
		{
			(*Found)->SetNewGizmoTransform(NewTransform);
		}
		else
		{
			(*Found)->ReinitializeGizmoTransform(NewTransform);
		}
	}
	else
	{
		UE_LOGF(LogTemp, Warning, "[SetGizmoTransform] Gizmo Identifier %ls could not be found", *Identifier);
	}
}


FTransform UScriptableInteractiveTool::GetGizmoTransform(
	FString Identifier)
{
	TObjectPtr<UCombinedTransformGizmo>* Found = Gizmos.Find(Identifier);
	if (Found != nullptr)
	{
		return (*Found)->GetGizmoTransform();
	}
	else
	{
		UE_LOGF(LogTemp, Warning, "[GetGizmoTransform] Gizmo Identifier %ls could not be found", *Identifier);
		return FTransform::Identity;
	}
}



void UScriptableInteractiveTool::OnGizmoTransformChanged_Handler(FString GizmoIdentifier, FTransform NewTransform)
{
	// forward to BlueprintImplementableEvent
	OnGizmoTransformChanged(GizmoIdentifier, NewTransform);
}

void UScriptableInteractiveTool::OnGizmoTransformStateChange_Handler(FString GizmoIdentifier, FTransform CurrentTransform, EScriptableToolGizmoStateChangeType ChangeType)
{
	// forward to BlueprintImplementableEvent
	OnGizmoTransformStateChange(GizmoIdentifier, CurrentTransform, ChangeType);
}




void UScriptableInteractiveTool::AddLogMessage(FText Message, bool bHighlighted)
{
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		if (bHighlighted)
		{
			UE_LOGF(LogScriptableInteractiveTool, Warning, "%ls", *Message.ToString() );
		}
		else
		{
			UE_LOGF(LogScriptableInteractiveTool, Display, "%ls", *Message.ToString() );
		}
	}
}

void UScriptableInteractiveTool::DisplayUserHelpMessage(FText Message)
{
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		ToolManager->DisplayMessage(Message, EToolMessageLevel::UserNotification);
	}
}

void UScriptableInteractiveTool::DisplayUserWarningMessage(FText Message)
{
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		ToolManager->DisplayMessage(Message, EToolMessageLevel::UserWarning);
	}
}

void UScriptableInteractiveTool::ClearUserMessages(bool bNotifications, bool bWarnings)
{
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		if (bNotifications)
		{
			ToolManager->DisplayMessage(FText(), EToolMessageLevel::UserNotification);
		}
		if (bWarnings)
		{
			ToolManager->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
		}
	}
}


void UScriptableInteractiveTool::SetTargets(TArray<TObjectPtr<UToolTarget>> TargetsIn)
{
	Targets = TargetsIn;
}

TArray<UToolTarget*> UScriptableInteractiveTool::GetToolTargets() const
{
	return Targets;
}


AActor* UScriptableInteractiveTool::GetDrawableGeometryActor() const
{
	return ToolDrawableGeometry ? ToolDrawableGeometry->GetActor() : nullptr;
}

UScriptableToolLineSet* UScriptableInteractiveTool::GetDefaultLineSet() const
{
	return DefaultLineSet;
}


UScriptableToolLineSet* UScriptableInteractiveTool::AddLineSet()
{
	TObjectPtr<UScriptableToolLineSet> LineSet = NewObject<UScriptableToolLineSet>();
	LineSet->Initialize(ToolDrawableGeometry);
	LineSets.Add( LineSet );
	return LineSet;
}

UScriptableToolPointSet* UScriptableInteractiveTool::GetDefaultPointSet() const
{
	return DefaultPointSet;
}


UScriptableToolPointSet* UScriptableInteractiveTool::AddPointSet()
{
	TObjectPtr<UScriptableToolPointSet> PointSet = NewObject<UScriptableToolPointSet>();
	PointSet->Initialize(ToolDrawableGeometry);
	PointSets.Add(PointSet);
	return PointSet;
}


UScriptableToolTriangleSet* UScriptableInteractiveTool::GetDefaultTriangleSet() const
{
	return DefaultTriangleSet;
}


UScriptableToolTriangleSet* UScriptableInteractiveTool::AddTriangleSet()
{
	TObjectPtr<UScriptableToolTriangleSet> TriangleSet = NewObject<UScriptableToolTriangleSet>();
	TriangleSet->Initialize(ToolDrawableGeometry);
	TriangleSets.Add(TriangleSet);
	return TriangleSet;
}


void UScriptableInteractiveTool::SetToolPanelToolHeaderWidget(UUserWidget* Widget)
{
	ToolPanelToolHeaderWidget = Widget;

	// Must invoke TakeWidget here to ensure that the underlying Slate widget is created
	// on this call, otherwise calls to set properties on the widget will fail during
	// OnScriptSetup.
	if (ToolPanelToolHeaderWidget)
	{
		ToolPanelToolHeaderWidget->TakeWidget();
	}
}

TSharedRef<SWidget> UScriptableInteractiveTool::GetToolPanelToolHeaderWidget() const
{
	return ToolPanelToolHeaderWidget ? ToolPanelToolHeaderWidget->TakeWidget() : SNullWidget::NullWidget;
}


void UScriptableInteractiveTool::SetOverlayWidget(UUserWidget* Widget, bool bMakeDraggable )
{
	UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore();

	UScriptableToolViewportWidgetAPI* ViewportWidgetAPI = ContextStore->FindContext< UScriptableToolViewportWidgetAPI>();
	if (ViewportWidgetAPI && Widget)
	{
		if (bMakeDraggable)
		{
			if (ViewportWidgetAPI->MakeDraggableWidgetFunc)
			{
				TSharedRef<SWidget> WrappedWidget = Widget->TakeDerivedWidget<ScriptableInteractiveToolLocals::SScriptableToolViewportOverlayDragContainer>([&ViewportWidgetAPI](UUserWidget* Widget, TSharedRef<SWidget> Content) {
				return SNew(ScriptableInteractiveToolLocals::SScriptableToolViewportOverlayDragContainer, Widget)
					[
						ViewportWidgetAPI->MakeDraggableWidgetFunc(Content)
					];
				});
				ViewportWidgetAPI->SetOverlayWidget(WrappedWidget);
			}
			else
			{
				ViewportWidgetAPI->SetOverlayWidget(Widget->TakeWidget());
			}
		}
		else
		{
			ViewportWidgetAPI->SetOverlayWidget(Widget->TakeWidget());
		}
	}
}

void UScriptableInteractiveTool::ClearOverlayWidget()
{
	UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore();

	UScriptableToolViewportWidgetAPI* ViewportWidgetAPI = ContextStore->FindContext< UScriptableToolViewportWidgetAPI>();
	if (ViewportWidgetAPI)
	{
		ViewportWidgetAPI->ClearOverlayWidget();
	}
}

void UScriptableInteractiveTool::SetFocusInViewport()
{
	UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore();

	UScriptableToolViewportFocusAPI* ViewportFocusAPI = ContextStore->FindContext< UScriptableToolViewportFocusAPI>();
	if (ViewportFocusAPI)
	{
		ViewportFocusAPI->SetFocusInViewport();
	}
}

FInputRayHit UScriptableToolsUtilityLibrary::MakeInputRayHit_Miss()
{
	return FInputRayHit();
}

FInputRayHit UScriptableToolsUtilityLibrary::MakeInputRayHit_MaxDepth()
{
	return FInputRayHit(TNumericLimits<float>::Max());
}

FInputRayHit UScriptableToolsUtilityLibrary::MakeInputRayHit(double HitDepth, UObject* OptionalHitObject)
{
	FInputRayHit Result(HitDepth);
	Result.HitObject = OptionalHitObject;
	return Result;
}

#undef LOCTEXT_NAMESPACE
