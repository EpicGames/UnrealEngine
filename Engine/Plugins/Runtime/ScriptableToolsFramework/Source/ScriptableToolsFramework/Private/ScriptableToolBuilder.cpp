// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptableToolBuilder.h"
#include "InteractiveToolManager.h"
#include "ScriptableInteractiveTool.h"
#include "ToolContextInterfaces.h"

#include "ToolTargetManager.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolBuilder)

bool UBaseScriptableToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return ToolClass.IsValid();
}


UInteractiveTool* UBaseScriptableToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UClass* UseClass = ToolClass.Get();

	UObject* NewToolObj = NewObject<UScriptableInteractiveTool>(SceneState.ToolManager, UseClass);
	check(NewToolObj != nullptr);
	UScriptableInteractiveTool* NewTool = Cast<UScriptableInteractiveTool>(NewToolObj);
	NewTool->SetTargetWorld(SceneState.World);
	check(NewTool != nullptr);
	return NewTool;
}

void UCustomScriptableToolBuilderContainer::Initialize(TObjectPtr<UCustomScriptableToolBuilderComponentBase> BuilderInstanceIn)
{
	BuilderInstance = BuilderInstanceIn;
}

bool UCustomScriptableToolBuilderContainer::CanBuildTool(const FToolBuilderState& SceneState) const
{
	bool bCanBuild = Super::CanBuildTool(SceneState);

	ICustomScriptableToolBuilderBaseInterface* Builder = Cast<ICustomScriptableToolBuilderBaseInterface>(BuilderInstance);
	bCanBuild = bCanBuild && Builder->CanBuildTool(SceneState);

	return bCanBuild;

}

UInteractiveTool* UCustomScriptableToolBuilderContainer::BuildTool(const FToolBuilderState& SceneState) const
{
	UInteractiveTool* NewToolObj = UBaseScriptableToolBuilder::BuildTool(SceneState);

	ICustomScriptableToolBuilderBaseInterface* Builder = Cast<ICustomScriptableToolBuilderBaseInterface>(BuilderInstance);
	Builder->SetupTool(SceneState, NewToolObj);

	return NewToolObj;
}

UScriptableToolTargetRequirements*
UScriptableToolTargetRequirements::BuildToolTargetRequirements(TArray<UClass*> RequirementInterfaces)
{
	TObjectPtr<UScriptableToolTargetRequirements> ScriptableToolRequirements = NewObject<UScriptableToolTargetRequirements>();

	for (UClass*& ClassPtr : RequirementInterfaces)
	{
		ScriptableToolRequirements->Requirements.Add(ClassPtr);
	}

	return ScriptableToolRequirements;
}


bool UCustomScriptableToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return OnCanBuildTool(SceneState.SelectedActors, SceneState.SelectedComponents);
}

bool UCustomScriptableToolBuilder::OnCanBuildTool_Implementation(const TArray<AActor*>& SelectedActors, const TArray<UActorComponent*>& SelectedComponents) const
{
	return true;
}

void UCustomScriptableToolBuilder::SetupTool(const FToolBuilderState& SceneState, UInteractiveTool* Tool) const
{
	UScriptableInteractiveTool* NewTool = Cast<UScriptableInteractiveTool>(Tool);

	OnSetupTool(NewTool, SceneState.SelectedActors, SceneState.SelectedComponents);
}

void UCustomScriptableToolBuilder::OnSetupTool_Implementation(UScriptableInteractiveTool* Tool, const TArray<AActor*>& SelectedActors, const TArray<UActorComponent*>& SelectedComponents) const
{

}


void UToolTargetScriptableToolBuilder::Initialize()
{
	Requirements = GetToolTargetRequirements();
}

UScriptableToolTargetRequirements* UToolTargetScriptableToolBuilder::GetToolTargetRequirements_Implementation() const
{
	return NewObject<UScriptableToolTargetRequirements>();
}

bool UToolTargetScriptableToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	int MatchingTargetCount = SceneState.TargetManager->CountSelectedAndTargetable(SceneState, Requirements->GetRequirements());
	return MatchingTargetCount >= Requirements->MinMatchingTargets && MatchingTargetCount <= Requirements->MaxMatchingTargets;
}

void UToolTargetScriptableToolBuilder::SetupTool(const FToolBuilderState& SceneState, UInteractiveTool* Tool) const
{
	UScriptableInteractiveTool* NewTool = Cast<UScriptableInteractiveTool>(Tool);
	TArray<TObjectPtr<UToolTarget>> Targets = SceneState.TargetManager->BuildAllSelectedTargetable(SceneState, Requirements->GetRequirements());
	const int NumTargets = Targets.Num();
	Targets.SetNum(FMath::Min(NumTargets, Requirements->MaxMatchingTargets));
	NewTool->SetTargets(Targets);
	OnSetupTool(NewTool);
}

void UToolTargetScriptableToolBuilder::OnSetupTool_Implementation(UScriptableInteractiveTool* Tool) const
{

}
