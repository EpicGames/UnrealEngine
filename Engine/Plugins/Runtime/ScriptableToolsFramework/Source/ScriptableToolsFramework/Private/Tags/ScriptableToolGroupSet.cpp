// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tags/ScriptableToolGroupSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolGroupSet)

void FScriptableToolGroupSet::SanitizeGroups()
{
	FGroupSet GroupSetCopy = Groups;
	GroupSetCopy.Remove(nullptr);
	Groups = GroupSetCopy;
}

