// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScriptableInteractiveTool.h"
#include "InputState.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "ScriptableClickDragTool.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API

class UClickDragInputBehavior;
class UMouseHoverBehavior;

/**
 * UScriptableClickDragTool is an extension of UScriptableInteractiveTool that supports
 * a "click-drag" interaction, ie capture the input device on press-down and collect any
 * device movement events until release. Hover is also supported. 
 * 
 * Note: Currently these interactions are only supported for mouse devices, or devices remapped to 
 * be a mouse by the UInputRouter. Also currently only the left-mouse-button is supported.
 */
UCLASS(MinimalAPI)
class UScriptableClickDragTool : public UScriptableInteractiveTool, public IClickDragBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	/**
	 * Enable Hover support API functions OnHoverHitTest / OnHoverBegin / OnHoverUpdate / OnHoverEnd for Mouse devices.
	 * Defaults to disabled. Cannot be changed dynamically after Setup runs.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SingleClick Tool Settings")
	bool bWantMouseHover = false;

	/**
	 * Allow Modifiers states to be updated during ClickDrag sequence. Defaults to enabled.
	 * Cannot be changed dynamically after Setup runs.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SingleClick Tool Settings")
	bool bUpdateModifiersDuringDrag = true;

	//
	// Click Support API
	//
public:

	UFUNCTION(BlueprintNativeEvent, Category = "ClickDrag Events")
	UE_API FInputRayHit TestIfCanBeginClickDrag(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual FInputRayHit TestIfCanBeginClickDrag_Implementation(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers);


	UFUNCTION(BlueprintNativeEvent, Category = "ClickDrag Events")
	UE_API void OnDragBegin(FInputDeviceRay StartPosition, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual void OnDragBegin_Implementation(FInputDeviceRay StartPosition, const FScriptableToolModifierStates& Modifiers);


	UFUNCTION(BlueprintNativeEvent, Category = "ClickDrag Events")
	UE_API void OnDragUpdatePosition(FInputDeviceRay NewPosition, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual void OnDragUpdatePosition_Implementation(FInputDeviceRay NewPosition, const FScriptableToolModifierStates& Modifiers);


	UFUNCTION(BlueprintNativeEvent, Category = "ClickDrag Events")
	UE_API void OnDragEnd(FInputDeviceRay EndPosition, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual void OnDragEnd_Implementation(FInputDeviceRay EndPosition, const FScriptableToolModifierStates& Modifiers);


	UFUNCTION(BlueprintNativeEvent, Category = "ClickDrag Events")
	UE_API void OnDragSequenceCancelled();

	UE_API virtual void OnDragSequenceCancelled_Implementation();

	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input")
	UE_API UPARAM(DisplayName="In ClickDrag State") bool InActiveClickDrag() const;

	//
	// Hover Support API
	//
public:

	/**
	 * OnHoverHitTest is called to allow the Tool to indicate if it would like to start consuing "hover" input at HoverPos.
	 * The Tool can return yes/no and a "hit depth", which will be used to determine if the Tool is given the active hover input stream.
	 * The default OnHoverHitTest implement always captures the hover.
	 * 
	 * Once the hover is accepted, the OnHoverBegin event will fire, and then a stream of OnHoverUpdate events. If at any point
	 * the hover is no longer relevant, OnHoverUpdate should return false. OnHoverEnd will be called at that point, or if 
	 * the hover is cancelled for any reason (eg mouse goes out of window, button is pressed, etc)
	 * 
	 * @param HoverPos the current position of the cursor/device, including both a 3D ray from the eye, and (optionally) a 2D mouse position
	 * @param Modifiers current modifier key/button state
	 * @return a FInputRayHit indicating a hit (true/false) and the hit depth along the hit-ray (0 for "always consume")
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Hover Events")
	UE_API FInputRayHit OnHoverHitTest(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual FInputRayHit OnHoverHitTest_Implementation(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	/**
	 * Begin a sequence of Hover input events 
	 * @param HoverPos the current position of the cursor/device, including both a 3D ray from the eye, and (optionally) a 2D mouse position
	 * @param Modifiers current modifier key/button state
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Hover Events")
	UE_API void OnHoverBegin(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual void OnHoverBegin_Implementation(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	/**
	 * Update an active Hover sequence. OnHoverUpdate is only ever called between OnHoverBegin and OnHoverEnd
	 * @param HoverPos the current position of the cursor/device, including both a 3D ray from the eye, and (optionally) a 2D mouse position
	 * @param Modifiers current modifier key/button state
	 * @return true to continue hovering, false to stop receiving additional hover events
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Hover Events")
	UE_API UPARAM(DisplayName="Continue Hovering") bool
	OnHoverUpdate(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual bool OnHoverUpdate_Implementation(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	/**
	 * Indicate that an active captured Hover sequence has ended. The device may no longer be in the viewport, so no position can be provided.
	 * @param Modifiers current modifier key/button state
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Hover Events")
	UE_API void OnHoverEnd(const FScriptableToolModifierStates& Modifiers);

	UE_API virtual void OnHoverEnd_Implementation(const FScriptableToolModifierStates& Modifiers);

	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input")
	UE_API UPARAM(DisplayName="In Hover State") bool InActiveHover() const;

	//
	// Modifer Buttons Support API
	//
public:

	/** @return true if the Shift key is currently held down */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API UPARAM(DisplayName="Shift Down") bool IsShiftDown() const;

	/** @return true if the Ctrl key is currently held down */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API UPARAM(DisplayName="Ctrl Down") bool IsCtrlDown() const;

	/** @return true if the Alt key is currently held down */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API UPARAM(DisplayName="Alt Down") bool IsAltDown() const;

	/** @return a struct containing the current Modifier key states */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API FScriptableToolModifierStates GetActiveModifiers();



public:
	UE_API virtual void Setup() override;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UClickDragInputBehavior> ClickDragBehavior;

	bool bInClickDrag = false;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UMouseHoverBehavior> MouseHoverBehavior;

	bool bInHover = false;

	bool bShiftModifier = false;
	bool bCtrlModifier = false;
	bool bAltModifier = false;

	// IClickDragBehaviorTarget API
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	UE_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	UE_API virtual void OnTerminateDragSequence() override;
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn);

	// IHoverBehaviorTarget API
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos);
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos);
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos);
	UE_API virtual void OnEndHover();

};

#undef UE_API
