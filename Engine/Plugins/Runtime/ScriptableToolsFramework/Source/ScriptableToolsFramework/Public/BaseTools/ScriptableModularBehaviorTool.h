// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScriptableInteractiveTool.h"
#include "InputState.h"
#include "InputBehavior.h"
#include "Behaviors/ScriptableToolBehaviorDelegates.h"
#include "ScriptableModularBehaviorTool.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API

class UScriptableToolSingleClickBehavior;
class UScriptableToolDoubleClickBehavior;
class UScriptableToolClickDragBehavior;
class UScriptableToolSingleClickOrDragBehavior;
class UScriptableToolMouseWheelBehavior;
class UScriptableToolClickSequenceBehavior;
class UScriptableToolMouseHoverBehavior;
class UScriptableToolKeyInputBehavior;

/**
 A Scriptable tool base blueprint class which provides support for user defined mouse interaction behaviors
 */ 
UCLASS(MinimalAPI)
class UScriptableModularBehaviorTool :	public UScriptableInteractiveTool
{
	GENERATED_BODY()

protected:


public:	

	/**
	*   Implements a standard "button-click"-style input behavior
	*   The state machine works as follows:
	*    1) on input-device-button-press, hit-test the target. If hit, begin capture
	*    2) on input-device-button-release, hit-test the target. If hit, call OnHitByClick(). If not hit, ignore click.
	*    
	*   The second hit-test is required to allow the click to be "cancelled" by moving away
	*   from the target. This is standard GUI behavior. You can disable this second hit test
	*   using the HitTestOnRelease property. This is strongly discouraged.
	* 
	*	@param IfHitByClick Test if hit by a click
	*	@param OnHitByClick Notify that click occurred
	*	@param CaptureCheck Only enable capture if returns true
	*	@param CapturePriority The priority is used to resolve situations where multiple behaviors want the same capture. Lower values, including negative, have higher priority.
	*	@param MouseButton Determines which mouse button the behavior captures on
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input", meta = (AdvancedDisplay = "CaptureCheck, CapturePriority, MouseButton, bHitTestOnRelease"))
	UE_API void AddSingleClickBehavior(
		const FTestIfHitByClickDelegate IfHitByClick,
		const FOnHitByClickDelegate OnHitByClick,
		const FMouseBehaviorModiferCheckDelegate CaptureCheck,
		int CapturePriority = 100,
		EScriptableToolMouseButton MouseButton = EScriptableToolMouseButton::LeftButton,
		bool bHitTestOnRelease = true
	);

	/**
	*   Implements a standard "button-click"-style input behavior
	*   The state machine works as follows:
	*    1) on input-device-button-press, hit-test the target. If hit, begin capture
	*    2) on input-device-button-release, hit-test the target. If hit, call OnHitByClick(). If not hit, ignore click.
	*    
	*   The second hit-test is required to allow the click to be "cancelled" by moving away
	*   from the target. This is standard GUI behavior. You can disable this second hit test
	*   using the HitTestOnRelease property. This is strongly discouraged.
	* 
	*	@param IfHitByClick Test if hit by a click
	*	@param OnHitByClick Notify that double click occurred
	*	@param CaptureCheck Only enable capture if returns true
	*	@param CapturePriority The priority is used to resolve situations where multiple behaviors want the same capture. Lower values, including negative, have higher priority.
	*	@param MouseButton Determines which mouse button the behavior captures on
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input", meta = (AdvancedDisplay = "CaptureCheck, CapturePriority, MouseButton, bHitTestOnRelease"))
	UE_API void AddDoubleClickBehavior(
		const FTestIfHitByClickDelegate IfHitByClick,
		const FOnHitByClickDelegate OnHitByClick,
		const FMouseBehaviorModiferCheckDelegate CaptureCheck,
		int CapturePriority = 100,
		EScriptableToolMouseButton MouseButton = EScriptableToolMouseButton::LeftButton,
		bool bHitTestOnRelease = true
	);

	/**
	*   Implements a standard "button-click-drag"-style input behavior.
	*	 
	*   The state machine works as follows:
	*      1) on input-device-button-press, call CanBeginClickDragSequence to determine if capture should begin
	*      2) on input-device-move, call OnClickDrag
	*      3) on input-device-button-release, call OnClickRelease
	*    
	*   If a ForceEndCapture occurs we call OnTerminateDragSequence   
	* 
	*   @param CanBeginClickDragSequence Test if target can begin click-drag interaction at this point
	*	@param OnClickPress Notify Target that click press occurred
	*	@param OnClickDrag Notify Target that input position has changed
	*	@param OnClickRelease Notify Target that click release occurred
	*	@param OnTerminateDragSequence Notify Target that click-drag sequence has been explicitly terminated (eg by escape key)
	*	@param CaptureCheck Only enable capture if returns true
	*	@param CapturePriority The priority is used to resolve situations where multiple behaviors want the same capture. Lower values, including negative, have higher priority.
	*	@param MouseButton Determines which mouse button the behavior captures on
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input", meta = (AdvancedDisplay = "CaptureCheck, CapturePriority, MouseButton, bUpdateModifiersDuringDrag"))
	UE_API void AddClickDragBehavior(		
		const FTestCanBeginClickDragSequenceDelegate CanBeginClickDragSequence,
		const FOnClickPressDelegate OnClickPress,
		const FOnClickDragDelegate OnClickDrag,
		const FOnClickReleaseDelegate OnClickRelease,
		const FOnTerminateDragSequenceDelegate OnTerminateDragSequence,
		const FMouseBehaviorModiferCheckDelegate CaptureCheck,
		int CapturePriority = 100,
		EScriptableToolMouseButton MouseButton = EScriptableToolMouseButton::LeftButton,
		bool bUpdateModifiersDuringDrag = false
	);

	/**
	* 
	*   SingleClickOrDragBehavior is a combination of a SingleClickBehavior and ClickDragBehavior,
	*   and allows for the common UI interaction where a click-and-release does one action, but if the mouse 
	*   is moved, then a drag interaction is started. For example click-to-select is often combined with
	*   a drag-marquee-rectangle in this way. This can be directly implemented with a ClickDragBehavior but
	*   requires the client to (eg) detect movement thresholds, etc. This class encapsulates all that state/logic.
	* 
	*   The ClickDistanceThreshold parameter determines how far the mouse must move (in whatever device units are in use)
	*   to switch from a click to drag interaction
	* 
	*   The bBeginDragIfClickTargetNotHit parameter determines if the drag interaction will be immediately initiated 
	*   if the initial 'click' mouse-down does not hit a valid clickable target. Defaults to true. 
	* 
	*	@param IfHitByClick Test if hit by a click
	*	@param OnHitByClick Notify that click occurred
	*   @param CanBeginClickDragSequence Test if target can begin click-drag interaction at this point
	*	@param OnClickPress Notify Target that click press occurred
	*	@param OnClickDrag Notify Target that input position has changed
	*	@param OnClickRelease Notify Target that click release occurred
	*	@param OnTerminateDragSequence Notify Target that click-drag sequence has been explicitly terminated (eg by escape key)
	*	@param CaptureCheck Only enable capture if returns true
	*	@param CapturePriority The priority is used to resolve situations where multiple behaviors want the same capture
	*	@param MouseButton Determines which mouse button the behavior captures on
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input", meta = (AdvancedDisplay = "CaptureCheck, CapturePriority, MouseButton, bBeginDragIfClickTargetNotHit, ClickDistanceThreshold"))
	UE_API void AddSingleClickOrDragBehavior(		
		FTestIfHitByClickDelegate IfHitByClick,
		FOnHitByClickDelegate OnHitByClick,
		FTestCanBeginClickDragSequenceDelegate CanBeginClickDragSequence,
		FOnClickPressDelegate OnClickPress,
		FOnClickDragDelegate OnClickDrag,
		FOnClickReleaseDelegate OnClickRelease,
		FOnTerminateDragSequenceDelegate OnTerminateDragSequence,
		FMouseBehaviorModiferCheckDelegate CaptureCheck,
		int CapturePriority = 100,
		EScriptableToolMouseButton MouseButton = EScriptableToolMouseButton::LeftButton,
		bool bBeginDragIfClickTargetNotHit = true,
		float ClickDistanceThreshold = 5.0
	);

	/**
	*	
	*   AddMouseWheelBehavior will run the event "OnMouseWheelScrollUp" and "OnMouseWheelScrollDown" each time the mouse wheel is rotated.
	*	It is up to the user to record these in variables or otherwise react to these events.
	* 
	*	@param TestShouldRespondToMouseWheel The result's bHit property determines whether the mouse wheel action will be captured. (Perhaps the mouse wheel only does something when mousing over some part of a mesh)
	*	@param OnMouseWheelScrollUp CurrentPos device position/ray at point where mouse wheel is engaged
	*	@param OnMouseWheelScrollDown CurrentPos device position/ray at point where mouse wheel is engaged
	*	@param CaptureCheck Only enable capture if returns true
	*	@param CapturePriority The priority is used to resolve situations where multiple behaviors want the same capture. Lower values, including negative, have higher priority.
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input", meta = (AdvancedDisplay = "CaptureCheck, CapturePriority"))
	UE_API void AddMouseWheelBehavior(		
		FTestShouldRespondToMouseWheelDelegate TestShouldRespondToMouseWheel,
		FOnMouseWheelScrollUpDelegate OnMouseWheelScrollUp,
		FOnMouseWheelScrollDownDelegate OnMouseWheelScrollDown,
		FMouseBehaviorModiferCheckDelegate CaptureCheck,
		int CapturePriority = 100
	);

	/**
	* 
	*   MultiClickSequenceBehavior implements a generic multi-click-sequence input behavior.
	*   For example this behavior could be used to implement a multi-click polygon-drawing interaction.
	*
	*   The internal state machine works as follows:
	*     1) on input-device-button-press, check if target wants to begin sequence. If so, begin capture.
	*     2) on button *release*, check if target wants to continue or terminate sequence
	*         a) if terminate, release capture
	*         b) if continue, do nothing (capture continues between presses)
	*
	*   The target will receive "preview" notifications (basically hover) during updates where there is
	*   not a release. This can be used to (eg) update a rubber-band selection end point
	* 
	*	@param OnBeginSequencePreview Notify Target device position has changed but a click sequence hasn't begun yet (eg for interactive previews)
	*	@param CanBeginClickSequence Test if target would like to begin sequence based on this click. Gets checked both on mouse down and mouse up.
	*	@param OnBeginClickSequence Notify Target about the first click in the sequence.
	*	@param OnNextSequencePreview Notify Target device position has changed but a click hasn't occurred yet (eg for interactive previews)
	*	@param OnNextSequenceClick  Notify Target about next click in sqeuence, returns false to terminate sequence
	*	@param OnTerminateClickSequence Notify Target that click sequence has been explicitly terminated (eg by escape key, cancel tool, etc).
	*	@param RequestAbortClickSequence Target overrides this and returns true if it wants to abort click sequence, checked every update
	*	@param CaptureCheck Only enable capture if returns true
	*   @param HoverCaptureCheck  Only enable hover capture if returns true
	*	@param CapturePriority The priority is used to resolve situations where multiple behaviors want the same capture
	*	@param MouseButton Determines which mouse button the behavior captures on
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input", meta = (AdvancedDisplay = "CaptureCheck, HoverCaptureCheck, CapturePriority, MouseButton"))
	UE_API void AddMultiClickSequenceBehavior(		
		FOnBeginSequencePreviewDelegate OnBeginSequencePreview,
		FCanBeginClickSequenceDelegate CanBeginClickSequence,
		FOnBeginClickSequenceDelegate OnBeginClickSequence,
		FOnNextSequencePreviewDelegate OnNextSequencePreview,
		FOnNextSequenceClickDelegate OnNextSequenceClick,
		FOnTerminateClickSequenceDelegate OnTerminateClickSequence,
		FRequestAbortClickSequenceDelegate RequestAbortClickSequence,
		FMouseBehaviorModiferCheckDelegate CaptureCheck,
		const FMouseBehaviorModiferCheckDelegate HoverCaptureCheck,
		int CapturePriority = 100,
		EScriptableToolMouseButton MouseButton = EScriptableToolMouseButton::LeftButton
	);

	/**
	* 
	*   AddMouseHoverBehavior implements a generic mouse hover behavior.
	*   For example, this can be used to change the "mode" of the current tool depending on what actor (if any) the mouse is hovering over.
	* 
	*	@param BeginHoverSequenceHitTest  Do hover hit-test. If this returns False, no further events below are executed.
	*	@param OnBeginHover Initialize hover sequence at given position
	*	@param OnUpdateHover Update active hover sequence with new input position
	*	@param OnEndHover Terminate active hover sequence
	*   @param HoverCaptureCheck  Only enable hover capture if returns true
	*	@param CapturePriority The priority is used to resolve situations where multiple behaviors want the same capture. Lower values, including negative, have higher priority.
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input", meta = (AdvancedDisplay = "CaptureCheck, CapturePriority"))
	UE_API void AddMouseHoverBehavior(
		FBeginHoverSequenceHitTestDelegate BeginHoverSequenceHitTest,
		FOnBeginHoverDelegate OnBeginHover,
		FOnUpdateHoverDelegate OnUpdateHover,
		FOnEndHoverDelegate OnEndHover,
		const FMouseBehaviorModiferCheckDelegate HoverCaptureCheck,
		int CapturePriority = 100
	);

	/**
	*	AddSingleKeyInputBehavior implements a generic keyboard key listener behavior.
	*   This is often used to add "hotkeys" or "shortcuts" to the tool.
	*   
	*   CapturePriority can be used to either give priority to "built in" viewport hotkeys (the default)
	*   or to have this behavior's key be the priority, using CapturePriority numbers lower than 50, for example.
	*	
	*	@param OnKeyPressed Callback when the target key is pressed.
	*	@param OnKeyReleased Callback when the target key is released
	*	@param OnForceEndCaptureFuncIn Callback when capture is ended prematurely, typically due to the viewport losing
	*	 focus, in which case the release callback will not be called.
	*	@param Key Target key to watch for
	*	@param CaptureCheck Only enable capture if returns true
	*	@param CapturePriority The priority is used to resolve situations where multiple behaviors want the same capture. Lower values, including negative, have higher priority.
	*/


	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input", meta = (AdvancedDisplay = "CaptureCheck, CapturePriority"))
	UE_API void AddSingleKeyInputBehavior(
		FOnKeyStateToggleDelegate OnKeyPressed,
		FOnKeyStateToggleDelegate OnKeyReleased,
		FOnForceEndCaptureDelegate_ScriptableTools OnForceEndCaptureFuncIn,
		FKey Key,
		const FMouseBehaviorModiferCheckDelegate CaptureCheck,
		int CapturePriority
	);

	UE_DEPRECATED(5.6, "Use overload that takes an OnForceEndCaptureFunc parameter as well")
	void AddSingleKeyInputBehavior(
		FOnKeyStateToggleDelegate OnKeyPressed,
		FOnKeyStateToggleDelegate OnKeyReleased,
		FKey Key,
		const FMouseBehaviorModiferCheckDelegate CaptureCheck,
		int CapturePriority)
	{
		AddSingleKeyInputBehavior(OnKeyPressed, OnKeyReleased, FOnForceEndCaptureDelegate_ScriptableTools(), Key, CaptureCheck, CapturePriority);
	}

	/**
	*	AddMultiKeyInputBehavior implements a generic keyboard multi key listener behavior
	*	
	*	@param OnKeyPressed  Callback when the target key(s) is pressed. Only triggers once if bRequireAllKeys is true.
	*	@param OnKeyReleased  Callback when the target key(s) is pressed. Only triggers once if bRequireAllKeys is true.
	*	@param OnForceEndCaptureFuncIn Callback when capture is ended prematurely, typically due to the viewport losing
	*	 focus, in which case the release callback will not be called.
	*	@param Keys Target keys to watch for
	*	@param bRequireAllKeys If true, all target keys must be pressed simultaniously to receive press/release events. Otherwise, any and all keys can trigger events.
	*	@param CaptureCheck Only enable capture if returns true
	*	@param CapturePriority The priority is used to resolve situations where multiple behaviors want the same capture. Lower values, including negative, have higher priority.
	*/


	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input", meta = (AdvancedDisplay = "CaptureCheck, CapturePriority"))
		UE_API void AddMultiKeyInputBehavior(
			FOnKeyStateToggleDelegate OnKeyPressed,
			FOnKeyStateToggleDelegate OnKeyReleased,
			FOnForceEndCaptureDelegate_ScriptableTools OnForceEndCaptureFuncIn,
			TArray<FKey> Keys,
			bool bRequireAllKeys,
			const FMouseBehaviorModiferCheckDelegate CaptureCheck,
			int CapturePriority
		);

	UE_DEPRECATED(5.6, "Use overload that takes an OnForceEndCaptureFunc parameter as well")
	void AddMultiKeyInputBehavior(
		FOnKeyStateToggleDelegate OnKeyPressed,
		FOnKeyStateToggleDelegate OnKeyReleased,
		TArray<FKey> Keys,
		bool bRequireAllKeys,
		const FMouseBehaviorModiferCheckDelegate CaptureCheck,
		int CapturePriority)
	{
		AddMultiKeyInputBehavior(OnKeyPressed, OnKeyReleased, FOnForceEndCaptureDelegate_ScriptableTools(), Keys, bRequireAllKeys, CaptureCheck, CapturePriority);
	}
private:

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray< TObjectPtr<UScriptableToolSingleClickBehavior> > SingleClickBehaviors;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray< TObjectPtr<UScriptableToolDoubleClickBehavior> > DoubleClickBehaviors;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray< TObjectPtr<UScriptableToolClickDragBehavior> > ClickDragBehaviors;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray< TObjectPtr<UScriptableToolSingleClickOrDragBehavior> > SingleClickOrDragBehaviors;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray< TObjectPtr<UScriptableToolMouseWheelBehavior> > MouseWheelBehaviors;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray< TObjectPtr<UScriptableToolClickSequenceBehavior> > MultiClickSequenceBehaviors;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray< TObjectPtr<UScriptableToolMouseHoverBehavior> > MouseHoverBehaviors;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray< TObjectPtr<UScriptableToolKeyInputBehavior> > KeyInputBehaviors;


	//
	// Modifer Buttons Support
	// 
	// We store these here, instead of in the behavior wrapper classes, to provide "global" access
	// via the blueprint methods regardless of what, if any, behaviors have been added to the tool.
	//
public:

	/** @return true if the Shift key is currently held down */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API UPARAM(DisplayName="Shift Down") bool IsShiftDown() const;

	/** @return true if the Ctrl key is currently held down */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API UPARAM(DisplayName="Ctrl Down") bool IsCtrlDown() const;

	/** @return true if the Alt key is currently held down */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API UPARAM(DisplayName="Alt Down") bool IsAltDown() const;

	/** @return a struct containing the current Modifier key states */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API FScriptableToolModifierStates GetActiveModifiers();

	UE_API void OnUpdateModifierState(int ModifierID, bool bIsOn);

private:

	bool bShiftModifier = false;
	bool bCtrlModifier = false;
	bool bAltModifier = false;


};



















#undef UE_API
