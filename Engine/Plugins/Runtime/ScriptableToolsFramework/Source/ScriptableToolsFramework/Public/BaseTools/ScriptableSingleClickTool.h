// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScriptableInteractiveTool.h"
#include "InputState.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"

#include "ScriptableSingleClickTool.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API

class USingleClickInputBehavior;
class UMouseHoverBehavior;

/**
 * UScriptableSingleClickTool is an extension of UScriptableInteractiveTool that supports
 * a "click" interaction, ie button press-down-and-release on an arbitrary Target (defined by the Tool).
 * Note that like a standard GUI click interaction, if the device is moved off the "Target" 
 * before the button is released, the click is cancelled. Note that in this context, the 
 * Target is defined implicitly by the Tool returning a "hit" for a given input device location,
 * rather than any specific geometry.
 * 
 * Hover is also supported. 
 * 
 * Note: Currently these interactions are only supported for mouse devices, or devices remapped to 
 * be a mouse by the UInputRouter. Also currently only the left-mouse-button is supported.
 */
UCLASS(MinimalAPI, Hidden)
class UScriptableSingleClickTool : public UScriptableInteractiveTool, public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	/**
	 * Enable Hover support API functions OnHoverHitTest / OnHoverBegin / OnHoverUpdate / OnHoverEnd for Mouse devices.
	 * Defaults to disabled.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "SingleClick Tool Settings")
	bool bWantMouseHover = false;

	//
	// Click Support API
	//
public:
	/**
	 * TestIfHitByClick is called to allow the Tool to indicate if it would like to consume a potential click at ClickPos.
	 * The Tool can return yes/no and a "hit depth", which will be used to determine if the Tool is given the click (ie if it has the nearest depth),
	 * at which point the OnHitByClick event will fire. The default TestIfHitByClick implementation always captures the click (at depth 0).
	 * 
	 * Note that this function will be called twice for a particular hit - once on mouse-down and once on mouse-up, and if it returns no-hit on mouse-up, 
	 * the click will be "cancelled", similar to standard GUI click interactions.
	 * 
	 * @param ClickPos the position of the click, including both a 3D ray from the eye, and (optionally) a 2D mouse position
	 * @param Modifiers current modifier key/button state
	 * @return a FInputRayHit indicating a hit (true/false) and the hit depth along the hit-ray (0 for "always consume")
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Click Events")
	UE_API FInputRayHit TestIfHitByClick(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual FInputRayHit TestIfHitByClick_Implementation(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers);

	/**
	 * OnHitByClick is called when the Tool has indicated it would like to consume a click event (via TestIfHitByClick), and the click was not cancelled
	 * or consumed by a nearer object. 
	 * @param ClickPos the position of the click, including both a 3D ray from the eye, and (optionally) a 2D mouse position
	 * @param Modifiers current modifier key/button state
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Click Events")
	UE_API void OnHitByClick(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual void OnHitByClick_Implementation(FInputDeviceRay ClickPos, const FScriptableToolModifierStates& Modifiers);


	//
	// Hover Support API
	//
public:

	/**
	 * OnHoverHitTest is called to allow the Tool to indicate if it would like to start consuing "hover" input at HoverPos.
	 * The Tool can return yes/no and a "hit depth", which will be used to determine if the Tool is given the active hover input stream.
	 * The default OnHoverHitTest implement always captures the hover.
	 * 
	 * Once the hover is accepted, the OnHoverBegin event will fire, and then a stream of OnHoverUpdate events. If at any point
	 * the hover is no longer relevant, OnHoverUpdate should return false. OnHoverEnd will be called at that point, or if 
	 * the hover is cancelled for any reason (eg mouse goes out of window, button is pressed, etc)
	 * 
	 * @param HoverPos the current position of the cursor/device, including both a 3D ray from the eye, and (optionally) a 2D mouse position
	 * @param Modifiers current modifier key/button state
	 * @return a FInputRayHit indicating a hit (true/false) and the hit depth along the hit-ray (0 for "always consume")
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Hover Events")
	UE_API FInputRayHit OnHoverHitTest(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual FInputRayHit OnHoverHitTest_Implementation(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	/**
	 * Begin a sequence of Hover input events 
	 * @param HoverPos the current position of the cursor/device, including both a 3D ray from the eye, and (optionally) a 2D mouse position
	 * @param Modifiers current modifier key/button state
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Hover Events")
	UE_API void OnHoverBegin(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual void OnHoverBegin_Implementation(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	/**
	 * Update an active Hover sequence. OnHoverUpdate is only ever called between OnHoverBegin and OnHoverEnd
	 * @param HoverPos the current position of the cursor/device, including both a 3D ray from the eye, and (optionally) a 2D mouse position
	 * @param Modifiers current modifier key/button state
	 * @return true to continue hovering, false to stop receiving additional hover events
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Hover Events")
	UE_API UPARAM(DisplayName="Continue Hovering") bool
	OnHoverUpdate(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	UE_API virtual bool OnHoverUpdate_Implementation(FInputDeviceRay HoverPos, const FScriptableToolModifierStates& Modifiers);

	/**
	 * Indicate that an active captured Hover sequence has ended. The device may no longer be in the viewport, so no position can be provided.
	 * @param Modifiers current modifier key/button state
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Hover Events")
	UE_API void OnHoverEnd(const FScriptableToolModifierStates& Modifiers);

	UE_API virtual void OnHoverEnd_Implementation(const FScriptableToolModifierStates& Modifiers);

	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Input")
	UE_API UPARAM(DisplayName="In Hover State") bool InActiveHover() const;

	//
	// Modifer Buttons Support API
	//
public:

	/** @return true if the Shift key is currently held down */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API UPARAM(DisplayName="Shift Down") bool IsShiftDown() const;

	/** @return true if the Ctrl key is currently held down */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API UPARAM(DisplayName="Ctrl Down") bool IsCtrlDown() const;

	/** @return true if the Alt key is currently held down */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API UPARAM(DisplayName="Alt Down") bool IsAltDown() const;

	/** @return a struct containing the current Modifier key states */
	UFUNCTION(BlueprintCallable, Category="ScriptableTool|Input")
	UE_API FScriptableToolModifierStates GetActiveModifiers();



public:
	UE_API virtual void Setup() override;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<USingleClickInputBehavior> SingleClickBehavior;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UMouseHoverBehavior> MouseHoverBehavior;

	bool bInHover = false;

	bool bShiftModifier = false;
	bool bCtrlModifier = false;
	bool bAltModifier = false;

	// IClickBehaviorTarget API
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos);
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos);
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn);

	// IHoverBehaviorTarget API
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos);
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos);
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos);
	UE_API virtual void OnEndHover();
};

#undef UE_API
