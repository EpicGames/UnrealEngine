// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScriptableInteractiveTool.h"
#include "InputState.h"

#include "ScriptableToolBehaviorDelegates.generated.h"

UENUM(BlueprintType)
enum class EScriptableToolMouseButton : uint8
{
	LeftButton,
	RightButton,
	MiddleButton
};

DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(bool, FMouseBehaviorModiferCheckDelegate, const FInputDeviceState&, InputDeviceState);

DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(FInputRayHit, FTestIfHitByClickDelegate, FInputDeviceRay, ClickPos, EScriptableToolMouseButton, MouseButton);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FOnHitByClickDelegate, FInputDeviceRay, ClickPos, FScriptableToolModifierStates, Modifiers, EScriptableToolMouseButton, MouseButton);

DECLARE_DYNAMIC_DELEGATE_RetVal_ThreeParams(FInputRayHit, FTestCanBeginClickDragSequenceDelegate, FInputDeviceRay, PressPos, FScriptableToolModifierStates, Modifiers, EScriptableToolMouseButton, MouseButton);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FOnClickPressDelegate, FInputDeviceRay, PressPos, FScriptableToolModifierStates, Modifiers, EScriptableToolMouseButton, MouseButton);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FOnClickDragDelegate, FInputDeviceRay, DragPos, FScriptableToolModifierStates, Modifiers, EScriptableToolMouseButton, MouseButton);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FOnClickReleaseDelegate, FInputDeviceRay, ReleasePos, FScriptableToolModifierStates, Modifiers, EScriptableToolMouseButton, MouseButton);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnTerminateDragSequenceDelegate, FScriptableToolModifierStates, Modifiers, EScriptableToolMouseButton, MouseButton);

DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(FInputRayHit, FTestShouldRespondToMouseWheelDelegate, FInputDeviceRay, CurrentPos);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnMouseWheelScrollUpDelegate, FInputDeviceRay, CurrentPos, FScriptableToolModifierStates, Modifiers);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnMouseWheelScrollDownDelegate, FInputDeviceRay, CurrentPos, FScriptableToolModifierStates, Modifiers);

DECLARE_DYNAMIC_DELEGATE_ThreeParams(FOnBeginSequencePreviewDelegate, FInputDeviceRay, ClickPos, FScriptableToolModifierStates, Modifiers, EScriptableToolMouseButton, MouseButton);
DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(bool, FCanBeginClickSequenceDelegate, FInputDeviceRay, ClickPos, EScriptableToolMouseButton, MouseButton);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FOnBeginClickSequenceDelegate, FInputDeviceRay, ClickPos, FScriptableToolModifierStates, Modifiers, EScriptableToolMouseButton, MouseButton);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FOnNextSequencePreviewDelegate, FInputDeviceRay, ClickPos, FScriptableToolModifierStates, Modifiers, EScriptableToolMouseButton, MouseButton);
DECLARE_DYNAMIC_DELEGATE_RetVal_ThreeParams(bool, FOnNextSequenceClickDelegate, FInputDeviceRay, ClickPos, FScriptableToolModifierStates, Modifiers, EScriptableToolMouseButton, MouseButton);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnTerminateClickSequenceDelegate, FScriptableToolModifierStates, Modifiers, EScriptableToolMouseButton, MouseButton);
DECLARE_DYNAMIC_DELEGATE_RetVal(bool, FRequestAbortClickSequenceDelegate);

DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(FInputRayHit, FBeginHoverSequenceHitTestDelegate, FInputDeviceRay, CurrentPos, FScriptableToolModifierStates, Modifiers);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnBeginHoverDelegate, FInputDeviceRay, CurrentPos, FScriptableToolModifierStates, Modifiers);
DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(bool, FOnUpdateHoverDelegate, FInputDeviceRay, CurrentPos, FScriptableToolModifierStates, Modifiers);
DECLARE_DYNAMIC_DELEGATE(FOnEndHoverDelegate);

DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnKeyStateToggleDelegate, FKey, Key, FScriptableToolModifierStates, Modifiers);
DECLARE_DYNAMIC_DELEGATE(FOnForceEndCaptureDelegate_ScriptableTools);

