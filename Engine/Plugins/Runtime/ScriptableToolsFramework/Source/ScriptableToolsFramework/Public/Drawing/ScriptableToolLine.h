// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ScriptableInteractiveTool.h"
#include "Drawing/LineSetComponent.h"


#include "ScriptableToolLine.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API


UCLASS(MinimalAPI, BlueprintType)
class UScriptableToolLine : public UObject
{
	GENERATED_BODY()

public:

	UE_API void SetLineID(int32 LineIDIn);
	UE_API int32 GetLineID() const;

	UE_API bool IsDirty() const;
	UE_API FRenderableLine GenerateLineDescription();

	/**
	 * Set the starting position of the line
	 * @param Start The position in space of the start of the line
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Lines")
	UE_API void SetLineStart(FVector Start);

	/**
	 * Set the ending position of the line
	 * @param End The position in space of the end of the line
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Lines")
	UE_API void SetLineEnd(FVector End);

	/**
	 * Set the starting and ending positions of the line
	 * @param Start The position in space of the start of the line
	 * @param End The position in space of the end of the line
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Lines")
	UE_API void SetLineEndPoints(FVector Start, FVector End);

	/**
	 * Set the line's color
	 * @param Color The color the line should be rendered as
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Lines")
	UE_API void SetLineColor(FColor Color);

	/**
	 * Set the line's thickness
	 * @param Thickness The visual thickness in pixels the line should be rendered as
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Lines")
	UE_API void SetLineThickness(float Thickness);

	/**
	 * Set the line's depth bias. The depth bias controls small micro adjustments in effective
	 * displacement towards or away from the camera, to prevent the line from causing clipping
	 * or z-fighting with other objects in the scene it's overlaid directly on top of.
	 * @param DepthBias The depth bias adjustment for the line
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Lines")
	UE_API void SetLineDepthBias(float DepthBias);

private:

	FRenderableLine LineDescription;

	UPROPERTY()
	bool bIsDirty = false;

	UPROPERTY()
	int32 LineID;

};

#undef UE_API
