// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScriptableInteractiveTool.h"
#include "Drawing/LineSetComponent.h"

#include "ScriptableToolLineSet.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API


class UScriptableToolLine;
class UPreviewGeometry;

UCLASS(MinimalAPI, BlueprintType)
class UScriptableToolLineSet : public UObject
{
	GENERATED_BODY()

public:

	UE_API void Initialize(TObjectPtr<UPreviewGeometry> PreviewGeometry);

	UE_API void OnTick();


	/**
	 * Create and return a new line object. Users should save a reference to this object for future updates or removal from the set.
	 * @return The new line object added to the set.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|LineSet")
	UE_API UPARAM(DisplayName = "LineComponent") UScriptableToolLine* AddLine();

	/**
	 * Remove a specific line object from the set, removing it fromt the scene.
	 * @param Line A reference to a line to be removed from the set.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|LineSet")
	UE_API void RemoveLine(UScriptableToolLine* Line);


	/**
	 * Remove all current lines in the set.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|LineSet")
	UE_API void RemoveAllLines();

	/**
	 * Set the color of all lines in the set simultaneously.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|LineSet")
	UE_API void SetAllLinesColor(FColor Color);

	/**
	 * Set the thickness of all lines in the set simultaneously.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|LineSet")
	UE_API void SetAllLinesThickness(float Thickness);

protected:

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UPreviewGeometry> ToolDrawableGeometry = nullptr;

	UE_DEPRECATED(5.7, "Use WeakLineSet instead.")
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<ULineSetComponent> LineSet = nullptr;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TWeakObjectPtr<ULineSetComponent> WeakLineSet = nullptr;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray<TObjectPtr<UScriptableToolLine>> LineComponents;

};

#undef UE_API
