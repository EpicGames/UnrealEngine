// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ScriptableInteractiveTool.h"
#include "Drawing/PointSetComponent.h"


#include "ScriptableToolPoint.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API


UCLASS(MinimalAPI, BlueprintType)
class UScriptableToolPoint : public UObject
{
	GENERATED_BODY()

public:

	UE_API void SetPointID(int32 PointIDIn);
	UE_API int32 GetPointID() const;

	UE_API bool IsDirty() const;
	UE_API FRenderablePoint GeneratePointDescription();

	/**
	 * Set the position of the point
	 * @param Position The position in space of the point
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Points")
	UE_API void SetPointPosition(FVector Position);

	/**
	 * Set the point's color
	 * @param Color The color the point should be rendered as
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Points")
	UE_API void SetPointColor(FColor Color);

	/**
	 * Set the point's size
	 * @param Size The visual size in pixels the point should be rendered as
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Points")
	UE_API void SetPointSize(float Thickness);

	/**
	 * Set the point's depth bias. The depth bias controls small micro adjustments in effective
	 * displacement towards or away from the camera, to prevent the point from causing clipping
	 * or z-fighting with other objects in the scene it's overlaid directly on top of.
	 * @param DepthBias The depth bias adjustment for the point
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Points")
	UE_API void SetPointDepthBias(float DepthBias);

private:

	FRenderablePoint PointDescription;

	UPROPERTY()
	bool bIsDirty = false;

	UPROPERTY()
	int32 PointID;

};

#undef UE_API
