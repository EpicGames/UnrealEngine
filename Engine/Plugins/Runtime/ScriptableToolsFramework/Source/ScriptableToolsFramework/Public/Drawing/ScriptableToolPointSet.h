// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScriptableInteractiveTool.h"
#include "Drawing/PointSetComponent.h"

#include "ScriptableToolPointSet.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API


class UScriptableToolPoint;
class UPreviewGeometry;

UCLASS(MinimalAPI, BlueprintType)
class UScriptableToolPointSet : public UObject
{
	GENERATED_BODY()

public:

	UE_API void Initialize(TObjectPtr<UPreviewGeometry> PreviewGeometry);

	UE_API void OnTick();

	/**
	 * Create and return a new point object. Users should save a reference to this object for future updates or removal from the set.
	 * @return The new point object added to the set.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|PointSet")
	UE_API UPARAM(DisplayName = "PointComponent") UScriptableToolPoint* AddPoint();


	/**
	 * Remove a specific point object from the set, removing it from the scene.
	 * @param Point A reference to a point to be removed from the set.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|PointSet")
	UE_API void RemovePoint(UScriptableToolPoint* Point);

	/**
	 * Remove all current points in the set.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|PointSet")
	UE_API void RemoveAllPoints();

	/**
	 * Set the color of all points in the set simultaneously.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|PointSet")
	UE_API void SetAllPointsColor(FColor Color);

	/**
	 * Set the size of all points in the set simultaneously.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|PointSet")
	UE_API void SetAllPointsSize(float Size);

protected:

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UPreviewGeometry> ToolDrawableGeometry = nullptr;

	UE_DEPRECATED(5.7, "Use WeakPointSet instead.")
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UPointSetComponent> PointSet = nullptr;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TWeakObjectPtr<UPointSetComponent> WeakPointSet = nullptr;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray<TObjectPtr<UScriptableToolPoint>> PointComponents;

};

#undef UE_API
