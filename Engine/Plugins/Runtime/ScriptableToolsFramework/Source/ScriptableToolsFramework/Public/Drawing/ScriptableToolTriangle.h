// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ScriptableInteractiveTool.h"
#include "Drawing/TriangleSetComponent.h"


#include "ScriptableToolTriangle.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API


UCLASS(MinimalAPI, BlueprintType)
class UScriptableToolTriangle : public UObject
{
	GENERATED_BODY()

public:

	UE_API UScriptableToolTriangle();

	UE_API void SetTriangleID(int32 TriangleIDIn);
	UE_API int32 GetTriangleID() const;

	UE_API bool IsDirty() const;
	UE_API FRenderableTriangle GenerateTriangleDescription();

	/*
	* Set the material of the triangle
	* 
	* @param Material The new material that should be assigned to the triangle
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Triangles")
	UE_API void SetTriangleMaterial(UMaterialInterface* Material);

	/*
	* Set the points of the triangle
	*
	* @param A The position of the first corner of the triangle
	* @param B The position of the second corner of the triangle
	* @param C The position of the third corner of the triangle
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Triangles")
	UE_API void SetTrianglePoints(FVector A, FVector B, FVector C);

	/*
	* Set the UV coordinates of the triangle
	*
	* @param A The UV coordinate of the first corner of the triangle
	* @param B The UV coordinate of the second corner of the triangle
	* @param C The UV coordinate of the third corner of the triangle
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Triangles")
	UE_API void SetTriangleUVs(FVector2D A, FVector2D B, FVector2D C);

	/*
	* Set the normal values of the triangle
	*
	* @param A The normal of the first corner of the triangle
	* @param B The normal of the second corner of the triangle
	* @param C The normal of the third corner of the triangle
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Triangles")
	UE_API void SetTriangleNormals(FVector A, FVector B, FVector C);


	/*
	* Set the vertex colors of the triangle
	*
	* @param A The vertex color of the first corner of the triangle
	* @param B The vertex color of the second corner of the triangle
	* @param C The vertex color of the third corner of the triangle
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Triangles")
	UE_API void SetTriangleColors(FColor A, FColor B, FColor C);

private:

	FRenderableTriangle TriangleDescription;

	UPROPERTY()
	bool bIsDirty = false;

	UPROPERTY()
	int32 TriangleID;

};

UCLASS(MinimalAPI, BlueprintType)
class UScriptableToolQuad : public UObject
{
	GENERATED_BODY()

public:

	UE_API UScriptableToolQuad();

	UE_API void SetTriangleAID(int32 TriangleIDIn);
	UE_API int32 GetTriangleAID() const;
	UE_API void SetTriangleBID(int32 TriangleIDIn);
	UE_API int32 GetTriangleBID() const;

	UE_API bool IsDirty() const;
	UE_API TPair<FRenderableTriangle, FRenderableTriangle> GenerateQuadDescription();

	/**
	 * Set the material of the quad
	 * @param Material The new material that should be assigned to the quad
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Quads")
	UE_API void SetQuadMaterial(UMaterialInterface* Material);

	/**
	 * Set the points of the quad
	 * @param A The position of the first corner of the quad
	 * @param B The position of the second corner of the quad
	 * @param C The position of the third corner of the quad
	 * @param D The position of the fourth corner of the quad
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Quads")
	UE_API void SetQuadPoints(FVector A, FVector B, FVector C, FVector D);

	/**
	 * Set the UV coordinates of the quad
	 * @param A The UV coordinate of the first corner of the quad
	 * @param B The UV coordinate of the second corner of the quad
	 * @param C The UV coordinate of the third corner of the quad
	 * @param D The UV coordinate of the fourth corner of the quad
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Quads")
	UE_API void SetQuadUVs(FVector2D A, FVector2D B, FVector2D C, FVector2D D);

	/**
	 * Set the normal values of the quad
	 * @param A The normal of the first corner of the quad
	 * @param B The normal of the second corner of the quad
	 * @param C The normal of the third corner of the quad
	 * @param D The normal of the fourth corner of the quad
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Quads")
	UE_API void SetQuadNormals(FVector A, FVector B, FVector C, FVector D);

	/**
	 * Set the vertex colors of the quad
	 * @param A The vertex color of the first corner of the quad
	 * @param B The vertex color of the second corner of the quad
	 * @param C The vertex color of the third corner of the quad
	 * @param D The vertex color of the fourth corner of the quad
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|Quads")
	UE_API void SetQuadColors(FColor A, FColor B, FColor C, FColor D);
private:

	FRenderableTriangle TriangleADescription;
	FRenderableTriangle TriangleBDescription;

	UPROPERTY()
	bool bIsDirty = false;

	UPROPERTY()
	int32 TriangleAID;

	UPROPERTY()
	int32 TriangleBID;

};

#undef UE_API
