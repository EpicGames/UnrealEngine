// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScriptableInteractiveTool.h"
#include "Drawing/TriangleSetComponent.h"

#include "ScriptableToolTriangleSet.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API


class UScriptableToolTriangle;
class UScriptableToolQuad;
class UPreviewGeometry;

UCLASS(MinimalAPI, BlueprintType)
class UScriptableToolTriangleSet : public UObject
{
	GENERATED_BODY()

public:

	UE_API void Initialize(TObjectPtr<UPreviewGeometry> PreviewGeometry);

	UE_API void OnTick();

	/**
	 * Create and return a new triangle object. Users should save a reference to this object for future updates or removal from the set.
	 * @return The new triangle object added to the set
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|TriangleSet")
	UE_API UPARAM(DisplayName = "TriangleComponent") UScriptableToolTriangle* AddTriangle();

	/**
	 * Create and return a new quad object. Quad objects are two paired triangles. Users should save a reference to this object for future updates or removal from the set.
	 * @return The new quad object added to the set
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|TriangleSet")
	UE_API UPARAM(DisplayName = "QuadComponent") UScriptableToolQuad* AddQuad();

	/**
	 * Remove a specific triangle object from the set, removing it from the scene.
	 * @param Triangle A reference to a triangle to be removed from the set.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|TriangleSet")
	UE_API void RemoveTriangle(UScriptableToolTriangle* Triangle);

	/**
	 * Remove a specific quad object from the set, removing it from the scene.
	 * @param Quad A reference to a quad to be removed from the set.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|TriangleSet")
	UE_API void RemoveQuad(UScriptableToolQuad* Quad);

	/**
	 * Remove all current triangles and quads in the set. 
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|TriangleSet")
	UE_API void RemoveAllFaces();

	/**
	 * Set the color of all triangles and quads in the set simultaneously .
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|TriangleSet")
	UE_API void SetAllTrianglesColor(FColor Color);

	/**
	 * Set the material of all triangles and quads in the set simultaneously .
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing|TriangleSet")
	UE_API void SetAllTrianglesMaterial(UMaterialInterface* Material);

protected:

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UPreviewGeometry> ToolDrawableGeometry = nullptr;

	UE_DEPRECATED(5.7, "Use WeakTriangleSet instead.")
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UTriangleSetComponent> TriangleSet = nullptr;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TWeakObjectPtr<UTriangleSetComponent> WeakTriangleSet = nullptr;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray<TObjectPtr<UScriptableToolTriangle>> TriangleComponents;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray<TObjectPtr<UScriptableToolQuad>> QuadComponents;

};

#undef UE_API
