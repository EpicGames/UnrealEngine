// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveTool.h"
#include "InteractiveGizmo.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "ScriptableToolBuilder.h"
#include "Tags/ScriptableToolGroupTag.h"
#include "Tags/ScriptableToolGroupSet.h"
#include "ScriptableInteractiveTool.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API

class UWorld;
class UCombinedTransformGizmo;
class UTransformProxy;
class UScriptableInteractiveTool;
class UBaseScriptableToolBuilder;
class FToolDataVisualizer;
class FCanvas;
class FSceneView;

class UPreviewGeometry;
class UTexture2D;

class UScriptableToolLineSet;
class UScriptableToolPointSet;
class UScriptableToolTriangleSet;
class UUserWidget;
class SWidget;

UENUM(BlueprintType)
enum class EToolsFrameworkOutcomePins : uint8
{
	Success,
	Failure
};


UENUM(BlueprintType)
enum class EScriptableToolShutdownType : uint8
{
	Complete = 0,
	AcceptCancel = 1
};

USTRUCT(BlueprintType)
struct FScriptableToolModifierStates
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modifiers)
	bool bShiftDown = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modifiers)
	bool bCtrlDown = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Modifiers)
	bool bAltDown = false;
};



UENUM(BlueprintType)
enum class EScriptableToolGizmoMode : uint8
{
	TranslationOnly = 0,
	RotationOnly = 1,
	ScaleOnly = 2,
	Combined = 3,
	FromViewportSettings = 4
};

UENUM(BlueprintType)
enum class EScriptableToolGizmoCoordinateSystem : uint8
{
	World = 0,
	Local = 1,
	FromViewportSettings = 2
};

UENUM(BlueprintType)
enum class EScriptableToolGizmoStateChangeType : uint8
{
	BeginTransform = 0,
	EndTransform = 1,
	UndoRedo = 2
};

UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EScriptableToolGizmoTranslation : uint8
{
	None = 0,

	TranslateAxisX = 1 << 1,
	TranslateAxisY = 1 << 2,
	TranslateAxisZ = 1 << 3,
	TranslatePlaneXY = 1 << 4,
	TranslatePlaneXZ = 1 << 5,
	TranslatePlaneYZ = 1 << 6,
	FreeTranslate = 1 << 7,

	All = 0xFF UMETA(Hidden)
};
ENUM_CLASS_FLAGS(EScriptableToolGizmoTranslation);


UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EScriptableToolGizmoRotation : uint8
{
	None = 0,

	RotateAxisX = 1 << 1,
	RotateAxisY = 1 << 2,
	RotateAxisZ = 1 << 3,

	FreeRotate = 1 << 7,

	All = 0xFF UMETA(Hidden)
};
ENUM_CLASS_FLAGS(EScriptableToolGizmoRotation);


UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EScriptableToolGizmoScale : uint8
{
	None = 0,

	ScaleAxisX = 1 << 1,
	ScaleAxisY = 1 << 2,
	ScaleAxisZ = 1 << 3,
	ScalePlaneXY = 1 << 4,
	ScalePlaneXZ = 1 << 5,
	ScalePlaneYZ = 1 << 6,
	ScaleUniform = 1 << 7,

	All = 0xFF UMETA(Hidden)
};
ENUM_CLASS_FLAGS(EScriptableToolGizmoScale);

UENUM(BlueprintType)
enum class EScriptableToolStartupRequirements : uint8
{
	/** No startup requirements needed. Tool can run any time. */
	None,
	/** A custom tool builder blueprint class that is configured with tool target requirements to filter selected objects. */
	ToolTarget,
	/** A custom tool builder blueprint class is provided to determine if the tool can start. Caution: OnCanBuildTool is run every tick, and may slow down editor performance. */
	Custom
};

/**
 * FScriptableToolGizmoOptions is a configuration struct passed to the CreateTRSGizmo function
 * of UScriptableInteractiveTool. 
 */
USTRUCT(BlueprintType)
struct FScriptableToolGizmoOptions
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Gizmo)
	EScriptableToolGizmoMode GizmoMode = EScriptableToolGizmoMode::Combined;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Gizmo, meta = (DisplayName = "Translation Elements", Bitmask, BitmaskEnum = "/Script/ScriptableToolsFramework.EScriptableToolGizmoTranslation",
	ValidEnumValues = "TranslateAxisX, TranslateAxisY, TranslateAxisZ, TranslatePlaneXY, TranslatePlaneXZ, TranslatePlaneYZ"))
	int32 TranslationParts = static_cast<int32>(EScriptableToolGizmoTranslation::All);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Gizmo, meta = (DisplayName = "Rotation Elements", Bitmask, BitmaskEnum = "/Script/ScriptableToolsFramework.EScriptableToolGizmoRotation",
	ValidEnumValues = "RotateAxisX, RotateAxisY, RotateAxisZ"))
	int32 RotationParts = static_cast<int32>(EScriptableToolGizmoRotation::All);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Gizmo, meta = (DisplayName = "Scale Elements", Bitmask, BitmaskEnum = "/Script/ScriptableToolsFramework.EScriptableToolGizmoScale",
	ValidEnumValues = "ScaleAxisX, ScaleAxisY, ScaleAxisZ, ScalePlaneXY, ScalePlaneXZ, ScalePlaneYZ"))
	int32 ScaleParts = static_cast<int32>(EScriptableToolGizmoScale::All);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Gizmo)
	EScriptableToolGizmoCoordinateSystem CoordSystem = EScriptableToolGizmoCoordinateSystem::Local;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Gizmo)
	bool bSnapTranslation = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Gizmo)
	bool bSnapRotation = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Gizmo)
	bool bRepositionable = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Gizmo)
	bool bAllowNegativeScaling = true;
};



/**
 * UScriptableTool_RenderAPI is helper Object that is created internally by a UScriptableInteractiveTool
 * to allow Blueprints to access basic 3D rendering functionality, in the context of a specific Tool.
 * The OnScriptRender event is called with an instance of this type.
 */
UCLASS(MinimalAPI, Transient)
class UScriptableTool_RenderAPI : public UObject
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Render")
	UE_API UPARAM(DisplayName="Render Object") UScriptableTool_RenderAPI* 
	DrawLine(FVector Start, FVector End, FLinearColor Color, float Thickness = 1.0, float DepthBias = 0.0f, bool bDepthTested = true);

	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Render")
	UE_API UPARAM(DisplayName="Render Object") UScriptableTool_RenderAPI* 
	DrawRectWidthHeightXY(FTransform Transform, double Width, double Height, FLinearColor Color, float LineThickness = 1.0, float DepthBias = 0.0f, bool bDepthTested = true, bool bCentered = true);


	// DrawPoint, DrawCircle, DrawViewFacingCircle
	// DrawWireCylinder, DrawWireBox, DrawSquare
	// PushTransform, PopTransform, SetTransform, PopAllTransforms

protected:
	friend class UScriptableInteractiveTool;

	FToolDataVisualizer* ActiveVisualizer = nullptr;
};

/**
 * UScriptableTool_HUDAPI is helper Object that is created internally by a UScriptableInteractiveTool
 * to allow Blueprints to access basic 2D rendering functionality, in the context of a specific Tool.
 * The OnScriptDrawHUD event is called with an instance of this type.
 */
UCLASS(MinimalAPI, Transient)
class UScriptableTool_HUDAPI : public UObject
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|DrawHUD", Meta = (Color = "(R=1.0,G=1.0,B=1.0,A=1.0)"))
	UE_API UPARAM(DisplayName="HUD Object") UScriptableTool_HUDAPI* 
	DrawTextAtLocation(FVector Location, FString String, FLinearColor Color, bool bCentered, float ShiftRowsY);

	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|DrawHUD", Meta = (Color = "(R=1.0,G=1.0,B=1.0,A=1.0)"))
	UE_API UPARAM(DisplayName="HUD Object") UScriptableTool_HUDAPI* 
	DrawTextArrayAtLocation(FVector Location, TArray<FString> Strings, FLinearColor Color, bool bCentered, float ShiftRowsY);

	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|DrawHUD")
	UE_API UPARAM(DisplayName="HUD Object") UScriptableTool_HUDAPI* 
	GetCanvasLocation(FVector Location, FVector2D& CanvasLocation );


	// DrawItem to draw line/etc (FCanvasLineItem, FCanvasBoxItem, FCanvasTriangleItem, FCanvasNGonItem)
	// DrawTile
	// DrawMaterial? via FCanvasTileItem

protected:
	friend class UScriptableInteractiveTool;
	FCanvas* ActiveCanvas = nullptr;
	const FSceneView* ActiveSceneView = nullptr;
};



/**
 * UScriptableInteractiveToolPropertySet is a Blueprintable extension of UInteractiveToolPropertySet.
 * This is a helper type meant to be used with UScriptableInteractiveTool. The intention is that
 * the "Settings" of a particular Tool are stored in UScriptableInteractiveToolPropertySet ("Property Set") instances.
 * The function UScriptableInteractiveTool::AddPropertySetOfType can be used to create and attach
 * a new Property Set. Then these settings can be automatically exposed by, for example, the Editor Mode UI, 
 * similar to (eg) Modeling Mode in the UE Editor.
 * 
 * In addition, UScriptableInteractiveTool has a set of functions like WatchFloatProperty, WatchBoolProperty, etc,
 * that can be used to detect and respond to changes in the Property Set. This works with both changes caused
 * by Editor UI (eg a Details Panel) as well as done directly in Blueprints or even C++ code. 
  * 
 * Note, however, that this Property Set mechanism completely optional, a Tool builder is free to use any
 * method whatsoever to store/modify Tool settings.
 */
UCLASS(MinimalAPI, Transient, Blueprintable)
class UScriptableInteractiveToolPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

	/** Access the Tool that owns this PropertySet */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool", Meta = (ExpandEnumAsExecs = "Outcome"))
	UE_API UScriptableInteractiveTool* GetOwningTool(
		EToolsFrameworkOutcomePins& Outcome);

	/** Toggle whether or not the named property should be displayed as a readonly field in the tool's details view. */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool")
	UE_API UPARAM(DisplayName = "Property Set") UScriptableInteractiveToolPropertySet* SetPropertyAsReadOnly(FString PropertyName, bool bReadOnly);

	/** Toggle whether or not the named property should be displayed at all as a field in the tool's details view. */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool")
	UE_API UPARAM(DisplayName = "Property Set") UScriptableInteractiveToolPropertySet* SetPropertyAsHidden(FString PropertyName, bool bHidden);

	UFUNCTION(BlueprintCallable, Category = "ScriptableTool")
	UE_API UPARAM(DisplayName = "Property Set") UScriptableInteractiveToolPropertySet* MakeTransactional(); 

	UE_API virtual UWorld* GetWorld() const override;	// must implement this for WorldContextObject dependent BP functions to show.

protected:
	TWeakObjectPtr<UScriptableInteractiveTool> ParentTool;
	friend class UScriptableInteractiveTool;
};



// these are delegates for the various property-watchers below
DECLARE_DYNAMIC_DELEGATE_TwoParams(FToolPropertyModifiedDelegate, UScriptableInteractiveToolPropertySet*, PropertySet, FString, PropertyName);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FToolFloatPropertyModifiedDelegate, UScriptableInteractiveToolPropertySet*, PropertySet, FString, PropertyName, double, NewValue);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FToolIntPropertyModifiedDelegate, UScriptableInteractiveToolPropertySet*, PropertySet, FString, PropertyName, int, NewValue);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FToolBoolPropertyModifiedDelegate, UScriptableInteractiveToolPropertySet*, PropertySet, FString, PropertyName, bool, bNewValue);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FToolEnumPropertyModifiedDelegate, UScriptableInteractiveToolPropertySet*, PropertySet, FString, PropertyName, uint8, NewValue);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FToolStringPropertyModifiedDelegate, UScriptableInteractiveToolPropertySet*, PropertySet, FString, PropertyName, FString, NewValue);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FToolFNamePropertyModifiedDelegate, UScriptableInteractiveToolPropertySet*, PropertySet, FString, PropertyName, FName, NewValue);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FToolObjectPropertyModifiedDelegate, UScriptableInteractiveToolPropertySet*, PropertySet, FString, PropertyName, UObject*, NewValue);


/**
 * UScriptableInteractiveTool is an extension of UInteractiveTool that allows the Tool functionality to be 
 * defined via Blueprints. 
 */
UCLASS(MinimalAPI, Transient, Blueprintable)
class UScriptableInteractiveTool : public UInteractiveTool
{
	GENERATED_BODY()
public:
	/** Name of this Tool, will be used in (eg) Toolbars */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Scriptable Tool Settings")
	FText ToolName;

	/** Long Name of this Tool, will be used in (eg) longer labels like the Accept/Cancel toolbar */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Scriptable Tool Settings", meta=(DisplayName="Long Name"))
	FText ToolLongName;

	/** Category of this Tool, will be used in (eg) Tool Palette Section headers */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Scriptable Tool Settings", meta=(DisplayName="Category"))
	FText ToolCategory;

	/** Tooltip used for this Tool in (eg) icons/etc */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Scriptable Tool Settings", meta=(DisplayName="Tooltip"))
	FText ToolTooltip;

	/**
	 * Relative Path to a custom Icon Image for this Tool. The Image file format must be png or svg. 
	 *
	 * The Image file must reside in the same Content folder hierarchy as contains the Tool Class (ie Blueprint asset), 
	 * i.e. either the Project Content folder or a Plugin Content folder.
	 * 
	 * So for example if the Tool BP is in a plugin named MyCustomTools, the icon must be in 
	 *      MyProject/Plugins/MyCustomTools/Content/<SubFolders>/MyToolIcon.png,
	 * and the relative path to use here would be <SubFolders>/MyToolIcon.png
	 */
	UE_DEPRECATED(5.7, "Use ToolIconTexture instead.")
	UPROPERTY(meta = (DisplayName = "Custom Icon Path", DeprecatedProperty, DeprecationMessage="Use ToolIconTexture instead."))
	FString CustomIconPath;

	UPROPERTY(EditDefaultsOnly, Category = "Scriptable Tool Settings", meta = (DisplayName = "Tool Icon"))
	TObjectPtr<UTexture2D> ToolIconTexture;

	/** A generic flag to indicate whether this Tool should be shown in the UE Editor. This may be interpreted in different ways */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Scriptable Tool Settings", meta=(DisplayName="Visible in Editor"))
	bool bShowToolInEditor = true;

	/** Specifies how the user exits this Tool, either Accept/Cancel-style or Complete-style */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Scriptable Tool Settings", meta=(DisplayName="Shutdown Type"))
	EScriptableToolShutdownType ToolShutdownType = EScriptableToolShutdownType::Complete;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Scriptable Tool Settings", meta = (DisplayName = "Tool Startup Requirements"))
	EScriptableToolStartupRequirements ToolStartupRequirements = EScriptableToolStartupRequirements::None;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Scriptable Tool Settings", meta = (DisplayName = "Tool Builder Class",
		                                                                                       EditCondition = "ToolStartupRequirements==EScriptableToolStartupRequirements::Custom",
																							   EditConditionHides,
																					           MustImplement = "/Script/ScriptableToolsFramework.CustomScriptableToolBuilderBaseInterface"))
	TSubclassOf<UCustomScriptableToolBuilder> CustomToolBuilderClass;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Scriptable Tool Settings", meta = (DisplayName = "Tool Builder Class",
		                                                                                       EditCondition = "ToolStartupRequirements==EScriptableToolStartupRequirements::ToolTarget",
		                                                                                       EditConditionHides,
		                                                                                       MustImplement="/Script/ScriptableToolsFramework.CustomScriptableToolBuilderBaseInterface"))
	TSubclassOf<UToolTargetScriptableToolBuilder> ToolTargetToolBuilderClass;



	/**
	 * Implement OnScriptSetup to do initial setup/configuration of the Tool, such as adding
	 * Property Sets, creating Gizmos, etc
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "ScriptableTool|Events")
	UE_API void OnScriptSetup();

	UE_API virtual void Setup() override;


	/**
	 * OnScriptTick is called every Editor Tick, ie basically every frame.
	 * Implement per-frame processing here.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "ScriptableTool|Events")
	UE_API void OnScriptTick(float DeltaTime);

	UE_API virtual void OnTick(float DeltaTime) override;


	/**
	 * CanAccept function. Implement this function for AcceptCancel Tools, and return
	 * true when it is valid for the Tool Accept button to be active. Defaults to always true.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "ScriptableTool|Events")
	UE_API bool OnScriptCanAccept() const;

	UE_API bool OnScriptCanAccept_Implementation() const;

	UE_API virtual bool HasAccept() const override;
	UE_API virtual bool HasCancel() const override;
	UE_API virtual bool CanAccept() const override;



	/**
	 * OnScriptShutdown is called when the Tool is shutting down. The ShutdownType defines
	 * what the Tool should do. For Complete-style Tools, there is no difference, but for
	 * Accept/Cancel-style Tools, on Accept the Tool should "commit" whatever it is previewing/etc,
	 * and on Cancel it should roll back / do-nothing. Which of these occurs is based on the
	 * ToolShutdownType property.
	 * 
	 * Tool Shutdown cannot be interrupted/aborted
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "ScriptableTool|Events")
	UE_API void OnScriptShutdown(EToolShutdownType ShutdownType);

	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	/**
	 * Request that the active Tool be shut down. The Tool can post this to shut itself down (eg in a Tool where
	 * the interaction paradigm is "click to do something and exit"). 
	 * @param bAccept if this is an Accept/Cancel Tool, Accept it, otherwise Cancel. Ignored for Complete-type Tools.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool", Meta = (Identifier="Settings"))
	UE_API void RequestToolShutdown(
		bool bAccept,
		bool bShowUserPopupMessage,
		FText UserMessage);



protected:
	UE_API virtual void PostInitProperties() override;


	// return instance of custom tool builder. Should only be called on CDO.
	UE_API virtual UBaseScriptableToolBuilder* GetNewCustomToolBuilderInstance(UObject* Outer);
	friend class UScriptableToolSet;


protected:
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TWeakObjectPtr<UWorld> TargetWorld;

public:
	UE_API virtual UWorld* GetWorld() const override;		// must implement this for WorldContextObject dependent BP functions to show.
	
	UE_API virtual void SetTargetWorld(UWorld* World);		// needs to be accessible to ToolBuilder

	/**
	 * Access the World this Tool is currently operating on.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool")
	UE_API UWorld* GetToolWorld();


	//
	// Render and DrawHUD support
	//
public:
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	/**
	 * OnScriptRender is called every frame. Use the RenderAPI object to draw
	 * various simple geometric elements like lines and points. This drawing
	 * is not very efficient but is useful for basic Tool visualization/feedback.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "ScriptableTool|Events")
	UE_API void OnScriptRender( UPARAM(DisplayName="Render Object") UScriptableTool_RenderAPI* RenderAPI);

	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	/**
	 * OnScriptDrawHUD is called every frame. Use the DrawHUDAPI to draw various
	 * simple HUD elements like text. This drawing is not very efficient but
	 * is useful for basic Tool visualization/feedback.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "ScriptableTool|Events")
	UE_API void OnScriptDrawHUD( UPARAM(DisplayName="HUD Object") UScriptableTool_HUDAPI* DrawHUDAPI);


protected:
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UScriptableTool_RenderAPI> RenderHelper;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UScriptableTool_HUDAPI> DrawHUDHelper;


	//
	// Property Set support
	//
public:

	/**
	 * Create a new Tool Property Set (ie BP subclass of UScriptableInteractiveToolPropertySet) with the given string Identifier
	 * and attach it to the Tool. The public variables of this Property Set object will appear in (eg) Mode Details Panels, etc.
	 * Multiple Property Sets of the same Type can be attached to a Tool, but each must have a unique Identifier.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets", Meta = (ExpandEnumAsExecs = "Outcome", Identifier="Settings"))
	UE_API UPARAM(DisplayName="New Property Set") UScriptableInteractiveToolPropertySet* 
	AddPropertySetOfType(
		TSubclassOf<UScriptableInteractiveToolPropertySet> PropertySetType,
		FString Identifier,
		EToolsFrameworkOutcomePins& Outcome);

	/**
	 * Remove a Property Set from the current Tool, found via it's unique Identifier.
	 * Unless the Property Set absolutely needs to stop existing, it is likely preferable to use SetPropertySetVisibleByName() to simply hide it.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets", Meta = (ExpandEnumAsExecs = "Outcome", Identifier="Settings"))
	UE_API void RemovePropertySetByName(
		FString Identifier,
		EToolsFrameworkOutcomePins& Outcome);

	/**
	 * Set the visibility of a Property Set that is paired with the given unique Identifier.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets", Meta = (Identifier="Settings"))
	UE_API void SetPropertySetVisibleByName(
		FString Identifier,
		bool bVisible);

	/**
	 * Force the Property Set associated with the given Identifier to be updated. 
	 * This may be necessary if you change the Property Set values directly via Blueprints, and want to
	 * see your changes reflected in (eg) a Details Panel showing the Tool Property values. 
	 * Currently these changes cannot be detected automatically, necessitating this function.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets", Meta = (Identifier="Settings"))
	UE_API void ForcePropertySetUpdateByName(
		FString Identifier);

	/**
	 * Restore the values of the specified PropertySet, optionally with a specific SaveKey string.
	 * This will have no effect unless SavePropertySetSettings() has been called on a compatible Property Set
	 * in the same Engine session (ie in a previous invocation of the Tool, or another Tool that uses the same Property Set)
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets", Meta = (Identifier="Settings"))
	UE_API UPARAM(DisplayName="Property Set") UScriptableInteractiveToolPropertySet* RestorePropertySetSettings(
		UScriptableInteractiveToolPropertySet* PropertySet,
		FString SaveKey);

	/**
	 * Save the values of the specified PropertySet, optionally with a specific SaveKey string.
	 * These saved values can be restored in future Tool invocations based on the SaveKey.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets", Meta = (Identifier="Settings"))
	UE_API UPARAM(DisplayName="Property Set") UScriptableInteractiveToolPropertySet* SavePropertySetSettings(
		UScriptableInteractiveToolPropertySet* PropertySet,
		FString SaveKey);



	//
	// Property Watcher support
	//
public:
	/**
	 * Watch a Float-valued Property for changes (double precision)
	 * @param PropertySet the Property Set which contains the desired Property to watch.
	 * @param PropertyName the string name of the Property in the given Property Set
	 * @param OnModified this delegate will be called if the Property value changes.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets")
	UE_API UPARAM(DisplayName="Property Set") UScriptableInteractiveToolPropertySet* WatchFloatProperty(
		UScriptableInteractiveToolPropertySet* PropertySet,
		FString PropertyName,
		const FToolFloatPropertyModifiedDelegate& OnModified );

	/**
	 * Watch an Integer-valued Property for changes
	 * @param PropertySet the Property Set which contains the desired Property to watch.
	 * @param PropertyName the string name of the Property in the given Property Set
	 * @param OnModified this delegate will be called if the Property value changes.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets")
	UE_API UPARAM(DisplayName="Property Set") UScriptableInteractiveToolPropertySet* WatchIntProperty(
		UScriptableInteractiveToolPropertySet* PropertySet,
		FString PropertyName,
		const FToolIntPropertyModifiedDelegate& OnModified );

	/**
	 * Watch a Bool-valued Property for changes
	 * @param PropertySet the Property Set which contains the desired Property to watch.
	 * @param PropertyName the string name of the Property in the given Property Set
	 * @param OnModified this delegate will be called if the Property value changes.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets")
	UE_API UPARAM(DisplayName="Property Set") UScriptableInteractiveToolPropertySet* WatchBoolProperty(
		UScriptableInteractiveToolPropertySet* PropertySet,
		FString PropertyName,
		const FToolBoolPropertyModifiedDelegate& OnModified );

	/**
	 * Watch an Enum-valued Property for changes. Note that in this case the OnModified
	 * delegate will be called with a uint8 integer, which can be cast back to the original Enum type.
	 * @param PropertySet the Property Set which contains the desired Property to watch.
	 * @param PropertyName the string name of the Property in the given Property Set
	 * @param OnModified this delegate will be called if the Property value changes.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets")
	UE_API UPARAM(DisplayName="Property Set") UScriptableInteractiveToolPropertySet* WatchEnumProperty(
		UScriptableInteractiveToolPropertySet* PropertySet,
		FString PropertyName,
		const FToolEnumPropertyModifiedDelegate& OnModified );

	/**
	 * Watch a String-valued Property for changes
	 * @param PropertySet the Property Set which contains the desired Property to watch.
	 * @param PropertyName the string name of the Property in the given Property Set
	 * @param OnModified this delegate will be called if the Property value changes.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets")
	UE_API UPARAM(DisplayName="Property Set") UScriptableInteractiveToolPropertySet* WatchStringProperty(
		UScriptableInteractiveToolPropertySet* PropertySet,
		FString PropertyName,
		const FToolStringPropertyModifiedDelegate& OnModified );

	/**
	 * Watch an (F)Name-valued Property for changes
	 * @param PropertySet the Property Set which contains the desired Property to watch.
	 * @param PropertyName the string name of the Property in the given Property Set
	 * @param OnModified this delegate will be called if the Property value changes.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets")
	UE_API UPARAM(DisplayName="Property Set") UScriptableInteractiveToolPropertySet* WatchNameProperty(
		UScriptableInteractiveToolPropertySet* PropertySet,
		FString PropertyName,
		const FToolFNamePropertyModifiedDelegate& OnModified );

	/**
	 * Watch an Object-valued Property for changes (ie UObject, UClass, etc)
	 * @param PropertySet the Property Set which contains the desired Property to watch.
	 * @param PropertyName the string name of the Property in the given Property Set
	 * @param OnModified this delegate will be called if the Property value changes.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets")
	UE_API UPARAM(DisplayName="Property Set") UScriptableInteractiveToolPropertySet* WatchObjectProperty(
		UScriptableInteractiveToolPropertySet* PropertySet,
		FString PropertyName,
		const FToolObjectPropertyModifiedDelegate& OnModified );

	/**
	 * Watch any Property in a PropertySet for changes. This function handles most basic
	 * properties, as well as Struct and Array properties, at some cost. The OnModified
	 * delegate cannot be passed the new property value, it must be fetched from the PropertySet.
	 * 
	 * Note also that in the case of Structs and Arrays, changes are currently detected by hashes,
	 * and there is always a small chance of hash collision.
	 * 
	 * @param PropertySet the Property Set which contains the desired Property to watch.
	 * @param PropertyName the string name of the Property in the given Property Set
	 * @param OnModified this delegate will be called if the Property value changes.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|PropertySets")
	UE_API UPARAM(DisplayName="Property Set") UScriptableInteractiveToolPropertySet* WatchProperty(
		UScriptableInteractiveToolPropertySet* PropertySet,
		FString PropertyName,
		const FToolPropertyModifiedDelegate& OnModified );

protected:
	TMap<FString, TWeakObjectPtr<UScriptableInteractiveToolPropertySet>> NamedPropertySets;

	TArray<FToolFloatPropertyModifiedDelegate> WatchFloatPropertyDelegates;
	TArray<FToolIntPropertyModifiedDelegate> WatchIntPropertyDelegates;
	TArray<FToolBoolPropertyModifiedDelegate> WatchBoolPropertyDelegates;
	TArray<FToolEnumPropertyModifiedDelegate> WatchEnumPropertyDelegates;
	TArray<FToolStringPropertyModifiedDelegate> WatchStringPropertyDelegates;
	TArray<FToolFNamePropertyModifiedDelegate> WatchFNamePropertyDelegates;
	TArray<FToolObjectPropertyModifiedDelegate> WatchObjectPropertyDelegates;


	enum class EAnyPropertyWatchTypes
	{
		Integer = 0,
		Bool = 1,
		Double = 2,
		Enum = 3,
		String = 4,
		FName = 5,
		Object = 6,

		Struct = 20,
		Array = 30,


		Unknown = 100
	};
	struct FAnyPropertyWatchInfo
	{
		EAnyPropertyWatchTypes KnownType = EAnyPropertyWatchTypes::Unknown;
		TArray<uint8> TempBuffer;
		FToolPropertyModifiedDelegate Delegate;
	};
	TArray<FAnyPropertyWatchInfo> WatchAnyPropertyInfo;

	// gizmo API
	//    - option to register delegate for specific gizmo?
	//    - some way to easily connect gizmo transform to a FTransform property?
public:

	/**
	 * Create a Translate/Rotate/Scale Gizmo with the given Options at the specified InitialTransform.
	 * The Gizmo must be given a unique Identifier, which will be used to access it in other functions.
	 * The Gizmo can be explicitly destroyed with DestroyTRSGizmo(), or it will be
	 * automatically destroyed when the Tool exits
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Gizmos", Meta = (ExpandEnumAsExecs = "Outcome", Identifier="Gizmo1"))
	UE_API void CreateTRSGizmo(
		FString Identifier,
		FTransform InitialTransform,
		FScriptableToolGizmoOptions GizmoOptions,
		EToolsFrameworkOutcomePins& Outcome);

	/**
	 * Destroy a created Gizmo by name Identifier
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Gizmos", Meta = (ExpandEnumAsExecs = "Outcome", Identifier="Gizmo1"))
	UE_API void DestroyTRSGizmo(
		FString Identifier,
		EToolsFrameworkOutcomePins& Outcome);

	/**
	 * Set an existing Gizmo visible/hidden based on its name Identifier
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Gizmos", Meta = (Identifier="Gizmo1") )
	UE_API void SetGizmoVisible(
		FString Identifier,
		bool bVisible);

	/**
	 * Update the Transform on the Gizmo specified by the name Identifier
	 * @param bUndoable if true, this transform change will be transacted into the undo/redo history, ie undoable
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Gizmos", Meta = (Identifier="Gizmo1") )
	UE_API void SetGizmoTransform(
		FString Identifier,
		FTransform NewTransform,
		bool bUndoable );

	/**
	 * Get the current Transform on the Gizmo specified by the name Identifier
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Gizmos", Meta = (Identifier="Gizmo1") )
	UE_API UPARAM(DisplayName="Transform") FTransform 
	GetGizmoTransform( FString Identifier);

	/**
	 * The OnGizmoTransformChanged event fires whenever the transform on any Gizmo created by CreateTRSGizmo() is modified.
	 * The GizmoIdentifier can be used to disambiguate multiple active Gizmos.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "ScriptableTool|Events")
	UE_API void OnGizmoTransformChanged(const FString& GizmoIdentifier, FTransform NewTransform);

	/**
	 * The OnGizmoTransformStateChange event fires whenever the user start/ends a Gizmo transform, or when an Undo/Redo event occurs.
	 * Note that when Undo/Redo occurs, OnGizmoTransformChanged will also fire.
	 * The GizmoIdentifier can be used to disambiguate multiple active Gizmos.
	 */
	UFUNCTION(BlueprintImplementableEvent, Category = "ScriptableTool|Events")
	UE_API void OnGizmoTransformStateChange(const FString& GizmoIdentifier, FTransform CurrentTransform, EScriptableToolGizmoStateChangeType ChangeType);


protected:
	// trying to avoid making a UStruct for this internal stuff
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TMap<FString, TObjectPtr<UCombinedTransformGizmo>> Gizmos;

	UE_API virtual void OnGizmoTransformChanged_Handler(FString GizmoIdentifier, FTransform NewTransform);
	UE_API virtual void OnGizmoTransformStateChange_Handler(FString GizmoIdentifier, FTransform CurrentTransform, EScriptableToolGizmoStateChangeType ChangeType);




public:
	// message API

	/**
	 * Append a Message to the UE Editor Log. 
	 * @param bHighlighted if true, the message is emitted as a Warning, otherwise as a Log (normal)
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Messaging")
	UE_API void AddLogMessage( FText Message, bool bHighlighted = false );

	/**
	 * Display a short Help message for the user, ie to guide them in Tool usage.
	 * In the UE5 Editor this message appears in the bottom bar of the window.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Messaging")
	UE_API void DisplayUserHelpMessage( FText Message );

	/**
	 * Display a Warning message to the user, ie to indicate a problem/issue occurred. 
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Messaging")
	UE_API void DisplayUserWarningMessage( FText Message );

	/**
	 * Clear any active message shown via DisplayUserHelpMessage and/or DisplayUserWarningMessage
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Messaging")
	UE_API void ClearUserMessages( bool bHelpMessage = true, bool bWarningMessage = true );


public:

	// TODO: hotkey API



public:

	// Tool Targets API

	UE_API void SetTargets(TArray<TObjectPtr<UToolTarget>> TargetsIn);

	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|ToolTargets")
	UE_API UPARAM(DisplayName = "Targets") TArray<UToolTarget*> GetToolTargets() const;

protected:

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray<TObjectPtr<UToolTarget>> Targets;


public:

	// Drawing API

	/**
	 * Return the actor that owns any line, point and triangle sets added to this tool.
	 * @return A reference to the actor object that owns drawable geometry sets.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing")
	UE_API AActor* GetDrawableGeometryActor() const;

	/**
	 * Retrieve the default line set object for the tool, used for drawing persistent line objects in the scene.
	 * @return A reference to the tool's default line set object
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing")
	UE_API UScriptableToolLineSet* GetDefaultLineSet() const;

	/**
	 * Create and return a new, independent line set, used for drawing persistent line objects in the scene.
	 * Users must save a reference to the created object for future access.
	 * @return A reference to a new line set object
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing")
	UE_API UScriptableToolLineSet* AddLineSet();

	/**
	 * Retrieve the default point set object for the tool, used for drawing persistent point objects in the scene.
	 * @return A reference to the tool's default point set object
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing")
	UE_API UScriptableToolPointSet* GetDefaultPointSet() const;

	/**
	 * Create and return a new, independent point set, used for drawing persistent point objects in the scene.
	 * Users must save a reference to the created object for future access.
	 * @return A reference to a new point set object
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing")
	UE_API UScriptableToolPointSet* AddPointSet();

	/**
	 * Retrieve the default triangle set object for the tool, used for drawing persistent triangle and quad objects in the scene.
	 * @return A reference to the tool's default triangle set object
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing")
	UE_API UScriptableToolTriangleSet* GetDefaultTriangleSet() const;

	/**
	 * Create and return a new, independent triangle set, used for drawing persistent triangle and quad objects in the scene.
	 * Users must save a reference to the created object for future access.
	 * @return A reference to a new triangle set object
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Drawing")
	UE_API UScriptableToolTriangleSet* AddTriangleSet();

protected:
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UScriptableToolLineSet> DefaultLineSet;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray<TObjectPtr<UScriptableToolLineSet> > LineSets;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UScriptableToolPointSet> DefaultPointSet;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray<TObjectPtr<UScriptableToolPointSet> > PointSets;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UScriptableToolTriangleSet> DefaultTriangleSet;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TArray<TObjectPtr<UScriptableToolTriangleSet> > TriangleSets;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<UPreviewGeometry> ToolDrawableGeometry = nullptr;


public:

	// Tool Panel Widget API

	/**
	 * Set a UUserWidget to display at the head of the Scriptable Tool panel details view.
	 * Will only take effect when set during OnScriptSetup.
	 * @param Widget The UUserWidget to set.
	 */
	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Widgets")
	UE_API void SetToolPanelToolHeaderWidget(UUserWidget* Widget);
	
	/**
	 * Retrieve an SWidget reference from the Scriptable Tool panel details view
	 * header widget.
	 * @return A reference to an SWidget
	 */
	UE_API TSharedRef<SWidget> GetToolPanelToolHeaderWidget() const;

private:

	UPROPERTY(Transient)
	TObjectPtr<UUserWidget> ToolPanelToolHeaderWidget;

public:
	
	// Widget Overlay API

	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Widgets")
	UE_API void SetOverlayWidget(UUserWidget* Widget, bool bMakeDraggable=true);

	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Widgets")
	UE_API void ClearOverlayWidget();

	// Tool Tagging Support


	UPROPERTY(EditAnywhere, Transient, Category = "Scriptable Tool Settings")
	FScriptableToolGroupSet GroupTags;

public:

	// Viewport Focus API

	UFUNCTION(BlueprintCallable, Category = "ScriptableTool|Viewport")
	UE_API void SetFocusInViewport();
};


UCLASS(MinimalAPI, meta = (ScriptName = "ScriptableTools_Util"))
class UScriptableToolsUtilityLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintPure, Category="ScriptableTool|Util")
	static UE_API UPARAM(DisplayName = "RayHit Result") FInputRayHit 
	MakeInputRayHit_Miss();

	UFUNCTION(BlueprintPure, Category="ScriptableTool|Util")
	static UE_API UPARAM(DisplayName = "RayHit Result") FInputRayHit 
	MakeInputRayHit_MaxDepth();

	UFUNCTION(BlueprintPure, Category="ScriptableTool|Util")
	static UE_API UPARAM(DisplayName = "RayHit Result") FInputRayHit 
	MakeInputRayHit(double HitDepth, UObject* OptionalHitObject);

};

#undef UE_API
