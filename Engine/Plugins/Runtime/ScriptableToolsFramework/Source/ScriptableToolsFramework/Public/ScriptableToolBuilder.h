// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveToolBuilder.h"
#include "ScriptableToolBuilder.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API


namespace ScriptableToolBuilderHelpers
{

}

class UScriptableInteractiveTool;

/**
 * UBaseScriptableToolBuilder is a trivial base UInteractiveToolBuilder for any UScriptableInteractiveTool subclass.
 * CanBuildTool will return true as long as the ToolClass is a valid UClass.
 */
UCLASS(MinimalAPI)
class UBaseScriptableToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:

	TWeakObjectPtr<UClass> ToolClass;

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};

UINTERFACE(MinimalAPI, BlueprintType)
class UCustomScriptableToolBuilderBaseInterface : public UInterface
{
	GENERATED_BODY()
};

class ICustomScriptableToolBuilderBaseInterface
{
	GENERATED_BODY()

public:

	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const = 0;
	virtual void SetupTool(const FToolBuilderState& SceneState, UInteractiveTool* Tool) const = 0;
};

UCLASS(MinimalAPI)
class UCustomScriptableToolBuilderComponentBase : public UObject
{
	GENERATED_BODY()

public:

};


UCLASS(MinimalAPI, Transient, Blueprintable, Hidden)
class UCustomScriptableToolBuilderContainer : public UBaseScriptableToolBuilder
{
	GENERATED_BODY()


public:

	UE_API void Initialize(TObjectPtr<UCustomScriptableToolBuilderComponentBase> BuilderInstanceIn);

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

private:

	UPROPERTY()
	TObjectPtr<UCustomScriptableToolBuilderComponentBase> BuilderInstance;
};

/*
*
*   Tool Builders for custom builder logic
*
*/

UCLASS(MinimalAPI, Transient, Blueprintable, Abstract)
class UCustomScriptableToolBuilder : public UCustomScriptableToolBuilderComponentBase, public ICustomScriptableToolBuilderBaseInterface
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintNativeEvent, Category = "ScriptableToolBuilder|Events")
	UE_API bool OnCanBuildTool(const TArray<AActor*>& SelectedActors, const TArray<UActorComponent*>& SelectedComponents) const;

	UE_API bool OnCanBuildTool_Implementation(const TArray<AActor*>& SelectedActors, const TArray<UActorComponent*>& SelectedComponents) const;

	UFUNCTION(BlueprintNativeEvent, Category = "ScriptableToolBuilder|Events")
	UE_API void OnSetupTool(UScriptableInteractiveTool* Tool, const TArray<AActor*>& SelectedActors, const TArray<UActorComponent*>& SelectedComponents) const;

	UE_API void OnSetupTool_Implementation(UScriptableInteractiveTool* Tool, const TArray<AActor*>& SelectedActors, const TArray<UActorComponent*>& SelectedComponents) const;

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void SetupTool(const FToolBuilderState& SceneState, UInteractiveTool* Tool) const override;
};


/*
*
*   Tool Builders for Tool Target support
* 
*/

UCLASS(MinimalAPI, Blueprintable)
class UScriptableToolTargetRequirements : public UObject
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category = "ScriptableToolBuilder|ToolTargets")
	static UE_API UPARAM(DisplayName = "New Target Requirements") UScriptableToolTargetRequirements*
	BuildToolTargetRequirements(TArray<UClass*> RequirementInterfaces);

	const FToolTargetTypeRequirements& GetRequirements() const { return Requirements; };

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Requirements")
	int MinMatchingTargets = 0;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Requirements")
	int MaxMatchingTargets = 1;

private:

	FToolTargetTypeRequirements Requirements;

};

UCLASS(MinimalAPI, Transient, Blueprintable, Abstract)
class UToolTargetScriptableToolBuilder: public UCustomScriptableToolBuilderComponentBase, public ICustomScriptableToolBuilderBaseInterface
{
	GENERATED_BODY()

public:

	UE_API void Initialize();

	UFUNCTION(BlueprintNativeEvent, Category = "Tool Targets")
	UE_API UScriptableToolTargetRequirements* GetToolTargetRequirements() const;

	UE_API virtual UScriptableToolTargetRequirements* GetToolTargetRequirements_Implementation() const;

	UFUNCTION(BlueprintNativeEvent, Category = "ScriptableToolBuilder|Events")
	UE_API void OnSetupTool(UScriptableInteractiveTool* Tool) const;

	UE_API void OnSetupTool_Implementation(UScriptableInteractiveTool* Tool) const;

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void SetupTool(const FToolBuilderState& SceneState, UInteractiveTool* Tool) const override;
	
private:

	UPROPERTY()
	TObjectPtr<UScriptableToolTargetRequirements> Requirements;

};

#undef UE_API
