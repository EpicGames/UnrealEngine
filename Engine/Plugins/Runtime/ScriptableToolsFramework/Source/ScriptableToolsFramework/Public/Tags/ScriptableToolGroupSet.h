// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Templates/SubclassOf.h"
#include "Tags/ScriptableToolGroupTag.h"

#include "ScriptableToolGroupSet.generated.h"

#define UE_API SCRIPTABLETOOLSFRAMEWORK_API

USTRUCT()
struct FScriptableToolGroupSet
{
	GENERATED_BODY()

	/**
	 * Note: This type needs to be specified explicitly for Groups because we can't use a typedef for a UPROPERTY.
	 */
	typedef TSet<TSubclassOf<UScriptableToolGroupTag>> FGroupSet;

public:

	bool Matches(const FScriptableToolGroupSet& OtherSet) const
	{
		return !GetGroups().Intersect(OtherSet.GetGroups()).IsEmpty();
	}

	void SetGroups(const FGroupSet& GroupsIn)
	{
		Groups = GroupsIn;
		SanitizeGroups();
	}

	const FGroupSet& GetGroups() const
	{
		return Groups;
	}

	FGroupSet& GetGroups()
	{
		return Groups;
	}

private:

	UE_API void  SanitizeGroups();

	UPROPERTY()
	TSet<TSubclassOf<UScriptableToolGroupTag>> Groups;
};

#undef UE_API
