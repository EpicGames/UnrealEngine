// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"

#include "ScriptableToolGroupTag.generated.h"

UCLASS(MinimalAPI, Abstract, Blueprintable, Const)
class UScriptableToolGroupTag
	: public UObject
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "General")
	FString Name;
};
