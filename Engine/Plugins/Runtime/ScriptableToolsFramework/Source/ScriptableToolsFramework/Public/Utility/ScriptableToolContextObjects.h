// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveToolManager.h"
#include "Widgets/SWidget.h"

#include "ScriptableToolContextObjects.generated.h"

/**
 * Base class for context objects used in the Scriptable Tools Framework.
 */
UCLASS(MinimalAPI)
class UScriptableToolContextObject : public UObject
{
	GENERATED_BODY()
public:
	/**
	 * Called by the mode when shutting context objects down, allowing them to do any cleanup.
	 * Initialization, on the other hand is usually done by some class-specific Initialize() method.
	 */
	virtual void Shutdown() {}

	/**
	 * Called whenever a tool is ended, for instance to let a context object remove listeners associated
	 * with that tool (it shouldn't have to do so, but may choose to for robustness).
	 */
	virtual void OnToolEnded(UInteractiveTool* DeadTool) {}
};


UCLASS(MinimalAPI)
class UScriptableToolViewportWidgetAPI : public UScriptableToolContextObject
{
	GENERATED_BODY()

public:

	void Initialize(TUniqueFunction<void(TSharedRef<SWidget> InOverlaidWidget)> ReplaceOverlaidWidgetFuncIn,
	               	TUniqueFunction<void(TSharedRef<SWidget> InOverlaidWidget)> ClearOverlaidWidgetFuncIn)
	{
		ReplaceOverlaidWidgetFunc = MoveTemp(ReplaceOverlaidWidgetFuncIn);
		ClearOverlaidWidgetFunc = MoveTemp(ClearOverlaidWidgetFuncIn);
	}

	void SetOverlayWidget(TSharedRef<SWidget> InOverlaidWidget)
	{
		if (CurrentOverlaidWidget.IsValid())
		{
			ClearOverlayWidget();
		}

		CurrentOverlaidWidget = InOverlaidWidget.ToWeakPtr();

		if (ReplaceOverlaidWidgetFunc)
		{
			ReplaceOverlaidWidgetFunc(InOverlaidWidget);
		}
	}
	void ClearOverlayWidget()
	{
		if (CurrentOverlaidWidget.IsValid())
		{
			if (ClearOverlaidWidgetFunc)
			{
				ClearOverlaidWidgetFunc(CurrentOverlaidWidget.Pin().ToSharedRef());
			}
			CurrentOverlaidWidget.Reset();
		}
	}

	virtual void Shutdown() override 
	{
		ClearOverlayWidget();
	}


	virtual void OnToolEnded(UInteractiveTool* DeadTool) override
	{
		ClearOverlayWidget();
	}

	TUniqueFunction<TSharedRef<SWidget>(TSharedRef<SWidget> Content)> MakeDraggableWidgetFunc;

private:

	TUniqueFunction<void(TSharedRef<SWidget> InOverlaidWidget)> ReplaceOverlaidWidgetFunc;
	TUniqueFunction<void(TSharedRef<SWidget> InOverlaidWidget)> ClearOverlaidWidgetFunc;

	TWeakPtr<SWidget> CurrentOverlaidWidget;
};

UCLASS(MinimalAPI)
class UScriptableToolViewportFocusAPI : public UScriptableToolContextObject
{
	GENERATED_BODY()

public:

	void Initialize(TUniqueFunction<void()> SetFocusInViewportFuncIn)
	{
		SetFocusInViewportFunc = MoveTemp(SetFocusInViewportFuncIn);
	}

	void SetFocusInViewport()
	{
		if (SetFocusInViewportFunc)
		{
			SetFocusInViewportFunc();
		}
	}

	virtual void Shutdown() override
	{
	}

	virtual void OnToolEnded(UInteractiveTool* DeadTool) override
	{
	}

private:

	TUniqueFunction<void()> SetFocusInViewportFunc;
};
