// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

SCRIPTABLETOOLSFRAMEWORK_API DECLARE_LOG_CATEGORY_EXTERN(LogScriptableTools, Log, All)